{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "RivermaxCore",
	"Description": "Base plugin exposing rivermax to engine",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"Hidden": "true",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RivermaxCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "RivermaxEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "RivermaxRendering",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux" ]
		}
	]
}