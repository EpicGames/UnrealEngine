// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RivermaxShaders.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/ColorUtils.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

Texture2D InputTexture;
SamplerState InputSampler;
float4x4 ColorTransform;
uint DoLinearToSrgb;
float OnePixelDeltaX;
float OnePixelDeltaY;
float InputPixelOffsetX;
float InputPixelOffsetY;
uint CapturedSizeX;
uint OutputTextureSizeX;
uint TotalElementCount;
RWTexture2D<float4> OutTexture;

/** This value needs to be reflected in GetComputeShaderGroupCount in RivermaxMediaUtils */
#define DISPATCH_GROUP_COUNT 512, 1, 1

struct FYUV8Bit422Buffer
{
	uint DWord0;
};
RWStructuredBuffer<FYUV8Bit422Buffer>	OutYUV4228bitBuffer;
StructuredBuffer<FYUV8Bit422Buffer>	InputYUV4228bitBuffer;

struct FYCbCr10Bit422
{
	uint DWord0;
	uint DWord1;
	uint DWord2;
	uint DWord3;
	uint DWord4;
};
RWStructuredBuffer<FYCbCr10Bit422>	OutYCbCrBuffer;
StructuredBuffer<FYCbCr10Bit422>	InputYCbCrBuffer;

struct FRGB8BitBuffer
{
	uint DWord0;
	uint DWord1;
	uint DWord2;
};
RWStructuredBuffer<FRGB8BitBuffer>	OutRGB8Buffer;
StructuredBuffer<FRGB8BitBuffer> InputRGB8Buffer;

struct FRGB10BitBuffer
{
	uint DWords[15];
};
RWStructuredBuffer<FRGB10BitBuffer>	OutRGB10Buffer;
StructuredBuffer<FRGB10BitBuffer> InputBuffer;

struct FRGB12BitBuffer
{
	uint DWords[9];
};
RWStructuredBuffer<FRGB12BitBuffer>	OutRGB12Buffer;
StructuredBuffer<FRGB12BitBuffer> InputRGB12Buffer;

struct FRGB16fBuffer
{
	uint DWords[3];
};
RWStructuredBuffer<FRGB16fBuffer> OutRGB16fBuffer;
StructuredBuffer<FRGB16fBuffer> InputRGB16fBuffer;



// Shader to convert from RGB to YUV 422 8 bits

[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToYUV8Bit422(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	const uint Index = ThreadId.x;

	const uint PixelCount = 2;
	uint3 YUV[PixelCount];

	// The 2 RGB needs to be converted to YUV. Each encoded pixel will use 2 RGB pixels (YnUnVnYn+1)
	const uint SourcePixelStart = ThreadId.x * PixelCount;
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		const float3 RGB = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb;
		YUV[PixelIndex] = RgbToYuv(RGB, ColorTransform, DoLinearToSrgb) * 255;
	}

	OutYUV4228bitBuffer[Index].DWord0 = (YUV[1].x << 24) | (YUV[0].z << 16) | (YUV[0].x << 8) | (YUV[0].y << 0);
}


// Shader to convert from RGB to YUV 422 10 bits packed little endian

[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToYUV10Bit422(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	const uint Index = ThreadId.x;

	const uint PixelCount = 8;
	uint3 YUV[PixelCount];

	// The 8 RGB needs to be converted to YUV. Each encoded pixel will use 2 RGB pixels (YnUnVnYn+1)
	const uint SourcePixelStart = ThreadId.x * PixelCount;
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		const float3 RGB = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb;
		YUV[PixelIndex] = RgbToYuv(RGB, ColorTransform, DoLinearToSrgb) * 1023;
	}

	// We write the 4 YUV pixels (40 bits * 4 = 160 bits = 5x32 bits) following the little endian packing 2110 expects
	OutYCbCrBuffer[Index].DWord0 = ((YUV[0].z & 0x3F) << 26) | ((YUV[1].x >> 8) << 24) | ((YUV[0].x & 0xF) << 20) | ((YUV[0].z >> 6) << 16) | ((YUV[0].y & 0x3) << 14) | ((YUV[0].x >> 4) << 8) | (YUV[0].y >> 2);
	OutYCbCrBuffer[Index].DWord1 = ((YUV[2].x & 0xF) << 28) | ((YUV[2].z >> 6) << 24) | ((YUV[2].y & 0x3) << 22) | ((YUV[2].x >> 4) << 16) | ((YUV[2].y >> 2) << 8) | (YUV[1].x & 0xFF);
	OutYCbCrBuffer[Index].DWord2 = ((YUV[4].y & 0x3) << 30) | ((YUV[4].x >> 4) << 24) | ((YUV[4].y >> 2) << 16) | ((YUV[3].x & 0xFF) << 8) | ((YUV[2].z & 0x3F) << 2) | (YUV[3].x >> 8);
	OutYCbCrBuffer[Index].DWord3 = ((YUV[6].y >> 2) << 24) | ((YUV[5].x & 0xFF) << 16) | ((YUV[4].z & 0x3F) << 10) | ((YUV[5].x >> 8) << 8) | ((YUV[4].x & 0xF) << 4) | (YUV[4].z >> 6);
	OutYCbCrBuffer[Index].DWord4 = ((YUV[7].x & 0xFF) << 24) | ((YUV[6].z & 0x3F) << 18) | ((YUV[7].x >> 8) << 16) | ((YUV[6].x & 0xF) << 12) | ((YUV[6].z >> 6) << 8) | ((YUV[6].y & 0x3) << 6) | (YUV[6].x >> 4);
}

// Shader to convert from RGBA to RGB10

[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToRGB10BitCS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Convert output buffer index to pixels
	const uint PixelCount = 16;
	const uint SourcePixelStart = (ThreadId.x * PixelCount);
	const uint Denormalizer = 1023;
	const uint Index = ThreadId.x;
	
	uint3 RGB[PixelCount];
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		RGB[PixelIndex] = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb * Denormalizer; 
		RGB[PixelIndex] = min(RGB[PixelIndex], Denormalizer);
	}

	// All this bit swizzling is to pack 3 channels of 10 bits with R being in LSB of the first 32 bits and subsequent channels following until we used all 16 pixels.
	// Example : RGB = (0x123, 0x245, 0x0AB)
	// Binary  : 0100 1000 1110 0100 0101 0011 0011 0100
	// Bytes   : 0x48 0xE4 0x53 0x34
	// 32 bits : 34 << 24 | 53 << 16 | E4 << 8 | 48 << 0
	// The bits of each channel not being next to each other seems weird but when you look at the byte ordering you can see it
	// The E in the second byte is the LSB of R and it's separated from the rest of R by the MSB of G.
	// We want 48 at the first adress so it goes in LSB of the 32 bits as it's written as little endian. 
	// 2110 on the wire will read it as big endian which will give us R going out first

	OutRGB10Buffer[Index].DWords[0] = ((RGB[0].z & 0x03F) << 26) | ((RGB[1].x >> 8)   << 24)   | ((RGB[0].y & 0xF)  << 20) | ((RGB[0].z >> 6)   << 16) | ((RGB[0].x & 0x3)  << 14) | ((RGB[0].y >> 4)   << 8)								| (RGB[0].x >> 2);
	OutRGB10Buffer[Index].DWords[1] = ((RGB[1].z & 0x00F) << 28) | ((RGB[2].x >> 6)   << 24)   | ((RGB[1].y & 0x3)  << 22) | ((RGB[1].z >> 4)   << 16)							  | ((RGB[1].y >> 2)   << 8)								| (RGB[1].x & 0xFF);
	OutRGB10Buffer[Index].DWords[2] = ((RGB[2].z & 0x003) << 30) | ((RGB[3].x >> 4)   << 24)								| ((RGB[2].z >> 2)   << 16)							  | ((RGB[2].y & 0xFF) << 8)  | ((RGB[2].x & 0x3F) << 2)  | (RGB[2].y >> 8);
	OutRGB10Buffer[Index].DWords[3] =							  ((RGB[4].x >> 2)   << 24)								| ((RGB[3].z & 0xFF) << 16) | ((RGB[3].y & 0x3F) << 10) | ((RGB[3].z >> 8)   << 8)  | ((RGB[3].x & 0xF)  << 4)  | (RGB[3].y >> 6);
	OutRGB10Buffer[Index].DWords[4] =							  ((RGB[5].x & 0xFF) << 24)   | ((RGB[4].z & 0x3F) << 18) | ((RGB[5].x >> 8)   << 16) | ((RGB[4].y & 0xF)  << 12) | ((RGB[4].z >> 6)   << 8)  | ((RGB[4].x & 0x3)  << 6)  | (RGB[4].y >> 4);
	
	OutRGB10Buffer[Index].DWords[5] = ((RGB[6].x & 0x03F) << 26) | ((RGB[6].y >> 8)   << 24)   | ((RGB[5].z & 0xF)  << 20) | ((RGB[6].x >> 6)   << 16) | ((RGB[5].y & 0x3)  << 14) | ((RGB[5].z >> 4)   << 8)								| (RGB[5].y >> 2);
	OutRGB10Buffer[Index].DWords[6] = ((RGB[7].x & 0x00F) << 28) | ((RGB[7].y >> 6)   << 24)   | ((RGB[6].z & 0x3)  << 22) | ((RGB[7].x >> 4)   << 16)							  | ((RGB[6].z >> 2)   << 8)								| (RGB[6].y & 0xFF);
	OutRGB10Buffer[Index].DWords[7] = ((RGB[8].x & 0x003) << 30) | ((RGB[8].y >> 4)   << 24)								| ((RGB[8].x >> 2)   << 16)							  | ((RGB[7].z & 0xFF) << 8)  | ((RGB[7].y & 0x3F) << 2)  | (RGB[7].z >> 8);
	OutRGB10Buffer[Index].DWords[8] =							  ((RGB[9].y >> 2)   << 24)								| ((RGB[9].x & 0xFF) << 16) | ((RGB[8].z & 0x3F) << 10) | ((RGB[9].x >> 8)   << 8)  | ((RGB[8].y & 0xF)  << 4)  | (RGB[8].z >> 6);
	OutRGB10Buffer[Index].DWords[9] =							  ((RGB[10].y & 0xFF) << 24)   | ((RGB[10].x & 0x3F) << 18) | ((RGB[10].y >> 8)   << 16) | ((RGB[9].z & 0xF)  << 12) | ((RGB[10].x >> 6)   << 8)  | ((RGB[9].y & 0x3)  << 6)  | (RGB[9].z >> 4);

	OutRGB10Buffer[Index].DWords[10] = ((RGB[11].y & 0x03F) << 26) | ((RGB[11].z >> 8)   << 24)  | ((RGB[11].x & 0xF)  << 20) | ((RGB[11].y >> 6)   << 16) | ((RGB[10].z & 0x3)  << 14) | ((RGB[11].x >> 4)   << 8)								| (RGB[10].z >> 2);
	OutRGB10Buffer[Index].DWords[11] = ((RGB[12].y & 0x00F) << 28) | ((RGB[12].z >> 6)   << 24)  | ((RGB[12].x & 0x3)  << 22) | ((RGB[12].y >> 4)   << 16)							  | ((RGB[12].x >> 2)   << 8)								| (RGB[11].z & 0xFF);
	OutRGB10Buffer[Index].DWords[12] = ((RGB[13].y & 0x003) << 30) | ((RGB[13].z >> 4)   << 24)								| ((RGB[13].y >> 2)   << 16)							  | ((RGB[13].x & 0xFF) << 8)  | ((RGB[12].z & 0x3F) << 2)  | (RGB[13].x >> 8);
	OutRGB10Buffer[Index].DWords[13] =							   ((RGB[14].z >> 2)   << 24)								| ((RGB[14].y & 0xFF) << 16) | ((RGB[14].x & 0x3F) << 10) | ((RGB[14].y >> 8)   << 8)  | ((RGB[13].z & 0xF)  << 4)  | (RGB[14].x >> 6);
	OutRGB10Buffer[Index].DWords[14] =							   ((RGB[15].z & 0xFF) << 24)  | ((RGB[15].y & 0x3F) << 18) | ((RGB[15].z >> 8)   << 16) | ((RGB[15].x & 0xF)  << 12) | ((RGB[15].y >> 6)   << 8)  | ((RGB[14].z & 0x3)  << 6)  | (RGB[15].x >> 4);
}

// Shader to convert from packed RGB10 to RGB10A2 texture

[numthreads(DISPATCH_GROUP_COUNT)]
void RGB10BitToRGBACS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 16 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 16;

	struct FOutputPixelData
	{
		uint2 UV;
		float4 RGB;
	};
	FOutputPixelData PixelData[OutputTexturePixelsPerThread];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		PixelData[PixelIndex].UV.x = ((SourcePixelStart + PixelIndex) % OutputTextureSizeX); 
		PixelData[PixelIndex].UV.y = ((SourcePixelStart + PixelIndex) / OutputTextureSizeX);
		PixelData[PixelIndex].RGB = float4(0,0,0,1);
	}

	const uint BufferIndex = ThreadId.x;

	// Used to normalize pixel values based on 10 bits input
	const float Mul = 1.0 / 1023.0;

	PixelData[0].RGB.x = ((((InputBuffer[BufferIndex].DWords[00] >>  0)  << 2) | ((InputBuffer[BufferIndex].DWords[00] >> 14) & 0x003)) & 0x3FF) * Mul;
	PixelData[0].RGB.y = ((((InputBuffer[BufferIndex].DWords[00] >>  8)  << 4) | ((InputBuffer[BufferIndex].DWords[00] >> 20) & 0x00F)) & 0x3FF) * Mul;
	PixelData[0].RGB.z = ((((InputBuffer[BufferIndex].DWords[00] >> 16)  << 6) | ((InputBuffer[BufferIndex].DWords[00] >> 26) & 0x03F)) & 0x3FF) * Mul;

	PixelData[1].RGB.x = ((((InputBuffer[BufferIndex].DWords[00] >> 24) << 8) | ((InputBuffer[BufferIndex].DWords[01] >> 00) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[1].RGB.y = ((((InputBuffer[BufferIndex].DWords[01] >>  8) << 2) | ((InputBuffer[BufferIndex].DWords[01] >> 22) & 0x003)) & 0x3FF) * Mul;
	PixelData[1].RGB.z = ((((InputBuffer[BufferIndex].DWords[01] >> 16) << 4) | ((InputBuffer[BufferIndex].DWords[01] >> 28) & 0x00F)) & 0x3FF) * Mul;

	PixelData[2].RGB.x = ((((InputBuffer[BufferIndex].DWords[01] >> 24) << 6) | ((InputBuffer[BufferIndex].DWords[02] >> 02) & 0x03F)) & 0x3FF) * Mul;
	PixelData[2].RGB.y = ((((InputBuffer[BufferIndex].DWords[02] >>  0) << 8) | ((InputBuffer[BufferIndex].DWords[02] >>  8) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[2].RGB.z = ((((InputBuffer[BufferIndex].DWords[02] >> 16) << 2) | ((InputBuffer[BufferIndex].DWords[02] >> 30) & 0x002)) & 0x3FF) * Mul;
	
	PixelData[3].RGB.x = ((((InputBuffer[BufferIndex].DWords[02] >> 24) << 4) | ((InputBuffer[BufferIndex].DWords[03] >> 04) & 0x00F)) & 0x3FF) * Mul;
	PixelData[3].RGB.y = ((((InputBuffer[BufferIndex].DWords[03] >> 00) << 6) | ((InputBuffer[BufferIndex].DWords[03] >> 10) & 0x03F)) & 0x3FF) * Mul;
	PixelData[3].RGB.z = ((((InputBuffer[BufferIndex].DWords[03] >>  8) << 8) | ((InputBuffer[BufferIndex].DWords[03] >> 16) & 0x0FF)) & 0x3FF) * Mul;
	
	PixelData[4].RGB.x = ((((InputBuffer[BufferIndex].DWords[03] >> 24) << 2) | ((InputBuffer[BufferIndex].DWords[04] >> 06) & 0x003)) & 0x3FF) * Mul;
	PixelData[4].RGB.y = ((((InputBuffer[BufferIndex].DWords[04] >> 00) << 4) | ((InputBuffer[BufferIndex].DWords[04] >> 12) & 0x00F)) & 0x3FF) * Mul;
	PixelData[4].RGB.z = ((((InputBuffer[BufferIndex].DWords[04] >>  8) << 6) | ((InputBuffer[BufferIndex].DWords[04] >> 18) & 0x03F)) & 0x3FF) * Mul;
	
	PixelData[5].RGB.x = ((((InputBuffer[BufferIndex].DWords[04] >> 16) << 8) | ((InputBuffer[BufferIndex].DWords[04] >> 24) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[5].RGB.y = ((((InputBuffer[BufferIndex].DWords[05] >> 00) << 2) | ((InputBuffer[BufferIndex].DWords[05] >> 14) & 0x003)) & 0x3FF) * Mul;
	PixelData[5].RGB.z = ((((InputBuffer[BufferIndex].DWords[05] >>  8) << 4) | ((InputBuffer[BufferIndex].DWords[05] >> 20) & 0x00F)) & 0x3FF) * Mul;

	PixelData[6].RGB.x = ((((InputBuffer[BufferIndex].DWords[05] >> 16) << 6) | ((InputBuffer[BufferIndex].DWords[05] >> 26) & 0x03F)) & 0x3FF) * Mul;
	PixelData[6].RGB.y = ((((InputBuffer[BufferIndex].DWords[05] >> 24) << 8) | ((InputBuffer[BufferIndex].DWords[06] >> 00) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[6].RGB.z = ((((InputBuffer[BufferIndex].DWords[06] >>  8) << 2) | ((InputBuffer[BufferIndex].DWords[06] >> 22) & 0x003)) & 0x3FF) * Mul;

	PixelData[7].RGB.x = ((((InputBuffer[BufferIndex].DWords[06] >> 16) << 4) | ((InputBuffer[BufferIndex].DWords[06] >> 28) & 0x00F)) & 0x3FF) * Mul;
	PixelData[7].RGB.y = ((((InputBuffer[BufferIndex].DWords[06] >> 24) << 6) | ((InputBuffer[BufferIndex].DWords[07] >> 02) & 0x03F)) & 0x3FF) * Mul;
	PixelData[7].RGB.z = ((((InputBuffer[BufferIndex].DWords[07] >>  0) << 8) | ((InputBuffer[BufferIndex].DWords[07] >>  8) & 0x0FF)) & 0x3FF) * Mul;

	PixelData[8].RGB.x = ((((InputBuffer[BufferIndex].DWords[07] >> 16) << 2) | ((InputBuffer[BufferIndex].DWords[07] >> 30) & 0x003)) & 0x3FF) * Mul;
	PixelData[8].RGB.y = ((((InputBuffer[BufferIndex].DWords[07] >> 24) << 4) | ((InputBuffer[BufferIndex].DWords[ 8] >> 04) & 0x00F)) & 0x3FF) * Mul;
	PixelData[8].RGB.z = ((((InputBuffer[BufferIndex].DWords[ 8] >>  0) << 6) | ((InputBuffer[BufferIndex].DWords[ 8] >> 10) & 0x03F)) & 0x3FF) * Mul;

	PixelData[9].RGB.x = ((((InputBuffer[BufferIndex].DWords[ 8] >>  8) << 8) | ((InputBuffer[BufferIndex].DWords[ 8] >> 16) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[9].RGB.y = ((((InputBuffer[BufferIndex].DWords[ 8] >> 24) << 2) | ((InputBuffer[BufferIndex].DWords[ 9] >> 06) & 0x003)) & 0x3FF) * Mul;
	PixelData[9].RGB.z = ((((InputBuffer[BufferIndex].DWords[ 9] >>  0) << 4) | ((InputBuffer[BufferIndex].DWords[ 9] >> 12) & 0x00F)) & 0x3FF) * Mul;

	PixelData[10].RGB.x = ((((InputBuffer[BufferIndex].DWords[ 9] >>  8) << 6) | ((InputBuffer[BufferIndex].DWords[ 9] >> 18) & 0x03F)) & 0x3FF) * Mul;
	PixelData[10].RGB.y = ((((InputBuffer[BufferIndex].DWords[ 9] >> 16) << 8) | ((InputBuffer[BufferIndex].DWords[ 9] >> 24) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[10].RGB.z = ((((InputBuffer[BufferIndex].DWords[10] >>  0) << 2) | ((InputBuffer[BufferIndex].DWords[10] >> 14) & 0x003)) & 0x3FF) * Mul;

	PixelData[11].RGB.x = ((((InputBuffer[BufferIndex].DWords[10] >>  8) << 4) | ((InputBuffer[BufferIndex].DWords[10] >> 20) & 0x00F)) & 0x3FF) * Mul;
	PixelData[11].RGB.y = ((((InputBuffer[BufferIndex].DWords[10] >> 16) << 6) | ((InputBuffer[BufferIndex].DWords[10] >> 26) & 0x03F)) & 0x3FF) * Mul;
	PixelData[11].RGB.z = ((((InputBuffer[BufferIndex].DWords[10] >>  24) << 8) | ((InputBuffer[BufferIndex].DWords[11] >> 00) & 0x0FF)) & 0x3FF) * Mul;
	
	PixelData[12].RGB.x = ((((InputBuffer[BufferIndex].DWords[11] >>  8) << 2) | ((InputBuffer[BufferIndex].DWords[11] >> 22) & 0x003)) & 0x3FF) * Mul;
	PixelData[12].RGB.y = ((((InputBuffer[BufferIndex].DWords[11] >> 16) << 4) | ((InputBuffer[BufferIndex].DWords[11] >> 28) & 0x00F)) & 0x3FF) * Mul;
	PixelData[12].RGB.z = ((((InputBuffer[BufferIndex].DWords[11] >> 24) << 6) | ((InputBuffer[BufferIndex].DWords[12] >> 02) & 0x03F)) & 0x3FF) * Mul;

	PixelData[13].RGB.x = ((((InputBuffer[BufferIndex].DWords[12] >>  0) << 8) | ((InputBuffer[BufferIndex].DWords[12] >>  8) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[13].RGB.y = ((((InputBuffer[BufferIndex].DWords[12] >> 16) << 2) | ((InputBuffer[BufferIndex].DWords[12] >> 30) & 0x003)) & 0x3FF) * Mul;
	PixelData[13].RGB.z = ((((InputBuffer[BufferIndex].DWords[12] >> 24) << 4) | ((InputBuffer[BufferIndex].DWords[13] >> 04) & 0x00F)) & 0x3FF) * Mul;

	PixelData[14].RGB.x = ((((InputBuffer[BufferIndex].DWords[13] >>  0) << 6) | ((InputBuffer[BufferIndex].DWords[13] >> 10) & 0x03F)) & 0x3FF) * Mul;
	PixelData[14].RGB.y = ((((InputBuffer[BufferIndex].DWords[13] >>  8) << 8) | ((InputBuffer[BufferIndex].DWords[13] >> 16) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[14].RGB.z = ((((InputBuffer[BufferIndex].DWords[13] >> 24) << 2) | ((InputBuffer[BufferIndex].DWords[14] >> 06) & 0x003)) & 0x3FF) * Mul;

	PixelData[15].RGB.x = ((((InputBuffer[BufferIndex].DWords[14] >>  0) << 4) | ((InputBuffer[BufferIndex].DWords[14] >> 12) & 0x00F)) & 0x3FF) * Mul;
	PixelData[15].RGB.y = ((((InputBuffer[BufferIndex].DWords[14] >>  8) << 6) | ((InputBuffer[BufferIndex].DWords[14] >> 18) & 0x03F)) & 0x3FF) * Mul;
	PixelData[15].RGB.z = ((((InputBuffer[BufferIndex].DWords[14] >> 16) << 8) | ((InputBuffer[BufferIndex].DWords[14] >> 24) & 0x0FF)) & 0x3FF) * Mul;

	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		OutTexture[PixelData[PixelIndex].UV] = PixelData[PixelIndex].RGB;
	}
}

// Shader to convert from RGBA to RGB8

[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToRGB8BitCS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Convert output buffer index to pixels
	const uint PixelCount = 4;
	const uint SourcePixelStart = (ThreadId.x * PixelCount);
	const uint Denormalizer = 255;
	const uint Index = ThreadId.x;
	
	uint3 RGB[PixelCount];
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		RGB[PixelIndex] = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb * Denormalizer; 
		RGB[PixelIndex] = min(RGB[PixelIndex], Denormalizer);
	}

	// We write the 4 RGB pixels (24 bits * 4 = 96 bits = 3x32 bits) following the packing 2110 expects
	OutRGB8Buffer[Index].DWord0 = (RGB[1].x << 24) | (RGB[0].z << 16) | (RGB[0].y << 8) | (RGB[0].x << 0);
	OutRGB8Buffer[Index].DWord1 = (RGB[2].y << 24) | (RGB[2].x << 16) | (RGB[1].z << 8) | (RGB[1].y << 0);
	OutRGB8Buffer[Index].DWord2 = (RGB[3].z << 24) | (RGB[3].y << 16) | (RGB[3].x << 8) | (RGB[2].z << 0);
}

// Shader to convert from packed RGB8 to RGBA8 texture

[numthreads(DISPATCH_GROUP_COUNT)]
void RGB8BitToRGBA8CS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 4 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 4;

	struct FOutputPixelData
	{
		uint2 UV;
		float4 RGB;
	};
	FOutputPixelData PixelData[OutputTexturePixelsPerThread];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		PixelData[PixelIndex].UV.x = ((SourcePixelStart + PixelIndex) % OutputTextureSizeX); 
		PixelData[PixelIndex].UV.y = ((SourcePixelStart + PixelIndex) / OutputTextureSizeX);
		PixelData[PixelIndex].RGB = float4(0,0,0,1);
	}

	const uint BufferIndex = ThreadId.x;

	// Used to normalize pixel values based on 8 bits input
	const float Mul = 1.0 / 255.0;

	PixelData[0].RGB.x = (((InputRGB8Buffer[BufferIndex].DWord0 >> 00) & 0xFF)) * Mul;
	PixelData[0].RGB.y = (((InputRGB8Buffer[BufferIndex].DWord0 >>  8) & 0xFF)) * Mul;
	PixelData[0].RGB.z = (((InputRGB8Buffer[BufferIndex].DWord0 >> 16) & 0xFF)) * Mul;

	PixelData[1].RGB.x = (((InputRGB8Buffer[BufferIndex].DWord0 >> 24) & 0xFF)) * Mul;
	PixelData[1].RGB.y = (((InputRGB8Buffer[BufferIndex].DWord1 >> 00) & 0xFF)) * Mul;
	PixelData[1].RGB.z = (((InputRGB8Buffer[BufferIndex].DWord1 >>  8) & 0xFF)) * Mul;

	PixelData[2].RGB.x = (((InputRGB8Buffer[BufferIndex].DWord1 >> 16) & 0xFF)) * Mul;
	PixelData[2].RGB.y = (((InputRGB8Buffer[BufferIndex].DWord1 >> 24) & 0xFF)) * Mul;
	PixelData[2].RGB.z = (((InputRGB8Buffer[BufferIndex].DWord2 >> 00) & 0xFF)) * Mul;
	
	PixelData[3].RGB.x = (((InputRGB8Buffer[BufferIndex].DWord2 >>  8) & 0xFF)) * Mul;
	PixelData[3].RGB.y = (((InputRGB8Buffer[BufferIndex].DWord2 >> 16) & 0xFF)) * Mul;
	PixelData[3].RGB.z = (((InputRGB8Buffer[BufferIndex].DWord2 >> 24) & 0xFF)) * Mul;
	

	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		OutTexture[PixelData[PixelIndex].UV] = PixelData[PixelIndex].RGB;
	}
}

// Shader to convert from RGBA to RGB12

[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToRGB12BitCS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Convert output buffer index to pixels
	const uint PixelCount = 8;
	const uint SourcePixelStart = (ThreadId.x * PixelCount);
	const uint Denormalizer = 4095;
	const uint Index = ThreadId.x;
	
	uint3 RGB[PixelCount];
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		RGB[PixelIndex] = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb * Denormalizer; 
		RGB[PixelIndex] = min(RGB[PixelIndex], Denormalizer);
	}

	OutRGB12Buffer[Index].DWords[0] =							  ((RGB[0].z >> 4)   << 24) |							 ((RGB[0].y & 0xFF)  << 16) |((RGB[0].x & 0xF)  << 12) | ((RGB[0].y >> 8)    << 8)								| (RGB[0].x >> 4);
	OutRGB12Buffer[Index].DWords[1] = ((RGB[1].y & 0x00F) << 28) | ((RGB[1].z >> 8)   << 24) |							 ((RGB[1].y >> 4)    << 16)							 | ((RGB[1].x & 0xFF)  << 8) | ((RGB[0].z & 0x00F) << 4)	| (RGB[1].x >> 8);
	OutRGB12Buffer[Index].DWords[2] =							  ((RGB[2].y & 0xFF) << 24) | ((RGB[2].x & 0xF) << 20)	| ((RGB[2].y >> 8)   << 16)							  | ((RGB[2].x >> 4)	  << 8)								| (RGB[1].z & 0xFF);
	OutRGB12Buffer[Index].DWords[3] =							  ((RGB[3].y >> 4)   << 24)								| ((RGB[3].x & 0xFF) << 16) | ((RGB[2].z & 0xF) << 12) | ((RGB[3].x >> 8)   << 8)								| (RGB[2].z >> 4);
	OutRGB12Buffer[Index].DWords[4] = ((RGB[4].x & 0x00F) << 28) | ((RGB[4].y >> 8)   << 24)								| ((RGB[4].x >> 4)   << 16)							  | ((RGB[3].z & 0xFF) << 8)  | ((RGB[3].y & 0x00F)  << 4)| (RGB[3].z >> 8);
	OutRGB12Buffer[Index].DWords[5] =							  ((RGB[5].x & 0xFF) << 24) | ((RGB[4].z & 0xF) << 20)	| ((RGB[5].x >> 8)   << 16)							  | ((RGB[4].z >> 4)   << 8)								| (RGB[4].y & 0xFF);
	OutRGB12Buffer[Index].DWords[6] =							  ((RGB[6].x >> 4)   << 24)								| ((RGB[5].z & 0xFF) << 16) | ((RGB[5].y & 0xF) << 12)  | ((RGB[5].z >> 8)   << 8)								| (RGB[5].y >> 4);
	OutRGB12Buffer[Index].DWords[7] = ((RGB[6].z & 0x00F) << 28) | ((RGB[7].x >> 8)   << 24)								| ((RGB[6].z >> 4)   << 16)							  | ((RGB[6].y & 0xFF) << 8)  | ((RGB[6].x & 0x00F) << 4) | (RGB[6].y >> 8);
	OutRGB12Buffer[Index].DWords[8] =							  ((RGB[7].z & 0xFF) << 24) | ((RGB[7].y & 0xF) << 20)	| ((RGB[7].z >> 8)   << 16)							  | ((RGB[7].y >> 4)   << 8)								| (RGB[7].x & 0xFF);
}

// Shader to convert from packed RGB12 to float RGBA texture

[numthreads(DISPATCH_GROUP_COUNT)]
void RGB12BitToRGBACS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 8 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 8;

	struct FOutputPixelData
	{
		uint2 UV;
		float4 RGB;
	};
	FOutputPixelData PixelData[OutputTexturePixelsPerThread];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		PixelData[PixelIndex].UV.x = ((SourcePixelStart + PixelIndex) % OutputTextureSizeX); 
		PixelData[PixelIndex].UV.y = ((SourcePixelStart + PixelIndex) / OutputTextureSizeX);
		PixelData[PixelIndex].RGB = float4(0,0,0,1);
	}

	const uint BufferIndex = ThreadId.x;

	// Used to normalize pixel values based on 12 bits input
	const float Mul = 1.0 / 4095.0;

	PixelData[0].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[00] >>  0)  << 4) | ((InputRGB12Buffer[BufferIndex].DWords[00] >> 12) & 0x00F)) & 0xFFF) * Mul;
	PixelData[0].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[00] >>  8)  << 8) | ((InputRGB12Buffer[BufferIndex].DWords[00] >> 16) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[0].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[00] >> 24)  << 4) | ((InputRGB12Buffer[BufferIndex].DWords[01] >> 04) & 0x00F)) & 0xFFF) * Mul;

	PixelData[1].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[01] >>  0) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[01] >>  8) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[1].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[01] >> 16) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[01] >> 28) & 0x00F)) & 0xFFF) * Mul;
	PixelData[1].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[01] >> 24) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[02] >> 00) & 0x0FF)) & 0xFFF) * Mul;

	PixelData[2].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[02] >>  8) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[02] >> 20) & 0x00F)) & 0xFFF) * Mul;
	PixelData[2].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[02] >> 16) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[02] >> 24) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[2].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[03] >>  0) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[03] >> 12) & 0x00F)) & 0xFFF) * Mul;
	
	PixelData[3].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[03] >>  8) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[03] >> 16) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[3].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[03] >> 24) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[04] >> 04) & 0x00F)) & 0xFFF) * Mul;
	PixelData[3].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[04] >>  0) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[04] >>  8) & 0x0FF)) & 0xFFF) * Mul;
	
	PixelData[4].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[04] >> 16) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[04] >> 28) & 0x00F)) & 0xFFF) * Mul;
	PixelData[4].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[04] >> 24) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[05] >>  0) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[4].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[05] >>  8) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[05] >> 20) & 0x00F)) & 0xFFF) * Mul;
	
	PixelData[5].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[05] >> 16) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[05] >> 24) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[5].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[06] >> 00) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[06] >> 12) & 0x00F)) & 0xFFF) * Mul;
	PixelData[5].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[06] >>  8) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[06] >> 16) & 0x0FF)) & 0xFFF) * Mul;

	PixelData[6].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[06] >> 24) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[07] >> 04) & 0x00F)) & 0xFFF) * Mul;
	PixelData[6].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[07] >>  0) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[07] >>  8) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[6].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[07] >>  16) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[07] >> 28) & 0x00F)) & 0xFFF) * Mul;

	PixelData[7].RGB.x = ((((InputRGB12Buffer[BufferIndex].DWords[07] >> 24) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[8] >> 00) & 0x0FF)) & 0xFFF) * Mul;
	PixelData[7].RGB.y = ((((InputRGB12Buffer[BufferIndex].DWords[8] >>  8) << 4) | ((InputRGB12Buffer[BufferIndex].DWords[8] >> 20) & 0x00F)) & 0xFFF) * Mul;
	PixelData[7].RGB.z = ((((InputRGB12Buffer[BufferIndex].DWords[8] >>  16) << 8) | ((InputRGB12Buffer[BufferIndex].DWords[8] >> 24) & 0x0FF)) & 0xFFF) * Mul;

	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		OutTexture[PixelData[PixelIndex].UV] = PixelData[PixelIndex].RGB;
	}
}

// Shader to convert from RGBA to RGB Float16
[numthreads(DISPATCH_GROUP_COUNT)]
void RGBToRGB16fBitCS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Convert output buffer index to pixels
	const uint PixelCount = 2;
	const uint SourcePixelStart = (ThreadId.x * PixelCount);
	const uint Index = ThreadId.x;
	
	uint3 IntRGB[PixelCount];
	for (uint PixelIndex = 0; PixelIndex < PixelCount; PixelIndex++)
	{
		const float BaseX = ((SourcePixelStart + PixelIndex) % CapturedSizeX) + InputPixelOffsetX; 
		const float BaseY = ((SourcePixelStart + PixelIndex) / CapturedSizeX) + InputPixelOffsetY;
		const float U = (BaseX + 0.5) * OnePixelDeltaX;
		const float V = (BaseY + 0.5) * OnePixelDeltaY;
		float3 RGB = InputTexture.SampleLevel(InputSampler, float2(U,V), 0).rgb; 
		IntRGB[PixelIndex].x = f32tof16(RGB.x);
		IntRGB[PixelIndex].y = f32tof16(RGB.y);
		IntRGB[PixelIndex].z = f32tof16(RGB.z);
	}

	OutRGB16fBuffer[Index].DWords[0]  = ((IntRGB[0].y & 0xFF) << 24) | ((IntRGB[0].y >> 8) << 16) | ((IntRGB[0].x & 0xFF) << 8) | ((IntRGB[0].x >> 8) << 0);
	OutRGB16fBuffer[Index].DWords[1]  = ((IntRGB[1].x & 0xFF) << 24) | ((IntRGB[1].x >> 8) << 16) | ((IntRGB[0].z & 0xFF) << 8) | ((IntRGB[0].z >> 8) << 0);
	OutRGB16fBuffer[Index].DWords[2]  = ((IntRGB[1].z & 0xFF) << 24) | ((IntRGB[1].z >> 8) << 16) | ((IntRGB[1].y & 0xFF) << 8) | ((IntRGB[1].y >> 8) << 0);
}

/**
 * This function extracts the 16bit (float) data stored in uint (32 bit)
 * Same used in EXRSwizzler. 
 */
float UintContainingFloatToFloat(in uint Value)
{
	int Sign = (Value >> 15) & 0x00000001;
	int Exponent = (Value >> 10) & 0x0000001f;
	int Mantissa = Value & 0x000003ff;

	Exponent = Exponent + (127 - 15);
	Mantissa = Mantissa << 13;

	return asfloat((Sign << 31) | (Exponent << 23) | Mantissa);
}

float GetEncodedFloatValue(in uint Value)
{
	uint LowerWord = Value & 0xFFFF;
	uint Swizzled = (LowerWord << 8);
	Swizzled += (LowerWord >> 8) & 0xFF;

	#if SM6_PROFILE
	return UintContainingFloatToFloat(Swizzled);
	#else
	return f16tof32(Swizzled);
	#endif
}

// Shader to convert from packed RGB16f to float RGBA texture

[numthreads(DISPATCH_GROUP_COUNT)]
void RGB16fBitToRGBACS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 2 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 2;

	struct FOutputPixelData
	{
		uint2 UV;
		float4 RGB;
	};
	FOutputPixelData PixelData[OutputTexturePixelsPerThread];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		PixelData[PixelIndex].UV.x = ((SourcePixelStart + PixelIndex) % OutputTextureSizeX); 
		PixelData[PixelIndex].UV.y = ((SourcePixelStart + PixelIndex) / OutputTextureSizeX);
		PixelData[PixelIndex].RGB = float4(0,0,0,1);
	}

	const uint BufferIndex = ThreadId.x;

	const uint Dword0 = InputRGB16fBuffer[BufferIndex].DWords[00];
	const uint Dword1 = InputRGB16fBuffer[BufferIndex].DWords[01];
	const uint Dword2 = InputRGB16fBuffer[BufferIndex].DWords[02];

	const uint IntRed0 = Dword0 & 0xFFFF;
	const uint IntGreen0 = (Dword0 >> 16) & 0xFFFF;
	const uint IntBlue0 = Dword1 & 0xFFFF;
	const uint IntRed1 = (Dword1 >> 16) & 0xFFFF;
	const uint IntGreen1 = Dword2 & 0xFFFF;
	const uint IntBlue1 = (Dword2 >> 16) & 0xFFFF;

	PixelData[0].RGB.x = GetEncodedFloatValue(IntRed0);
	PixelData[0].RGB.y = GetEncodedFloatValue(IntGreen0);
	PixelData[0].RGB.z = GetEncodedFloatValue(IntBlue0);
	PixelData[1].RGB.x = GetEncodedFloatValue(IntRed1);
	PixelData[1].RGB.y = GetEncodedFloatValue(IntGreen1);
	PixelData[1].RGB.z = GetEncodedFloatValue(IntBlue1);
	
	for (uint PixelIndex = 0; PixelIndex < OutputTexturePixelsPerThread; PixelIndex++)
	{
		OutTexture[PixelData[PixelIndex].UV] = PixelData[PixelIndex].RGB;
	}
}

// Shader to convert from YUV 422 8 bits to RGBA

[numthreads(DISPATCH_GROUP_COUNT)]
void YUV8Bit422ToRGBACS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 8 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 2;

	struct FOutputPixelData
	{
		uint2 UV1;
		uint2 UV2;
		float4 YUVY;
	};
	const uint PixelContainerSize = OutputTexturePixelsPerThread / 2;
	FOutputPixelData PixelData[PixelContainerSize];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint ArrayIndex = 0; ArrayIndex < PixelContainerSize; ArrayIndex++)
	{
		const uint PixelIndex = ArrayIndex * 2;
		PixelData[ArrayIndex].UV1.x = ((SourcePixelStart + (PixelIndex + 0)) % OutputTextureSizeX); 
		PixelData[ArrayIndex].UV1.y = ((SourcePixelStart + (PixelIndex + 0)) / OutputTextureSizeX);
		PixelData[ArrayIndex].UV2.x = ((SourcePixelStart + (PixelIndex + 1)) % OutputTextureSizeX); 
		PixelData[ArrayIndex].UV2.y = ((SourcePixelStart + (PixelIndex + 1)) / OutputTextureSizeX);

		PixelData[ArrayIndex].YUVY = float4(0,0,0,0);
	}

	// Input buffer indexing is simply thread ix
	const uint BufferIndex = ThreadId.x;

	// Used to normalize pixel values based on 8 bits input
	const float Mul = 1.0 / 255.0;
	
	PixelData[0].YUVY.y = ((InputYUV4228bitBuffer[BufferIndex].DWord0 >>  0) & 0xFF) * Mul;
	PixelData[0].YUVY.x = ((InputYUV4228bitBuffer[BufferIndex].DWord0 >>  8) & 0xFF) * Mul;
	PixelData[0].YUVY.z = ((InputYUV4228bitBuffer[BufferIndex].DWord0 >> 16) & 0xFF) * Mul;
	PixelData[0].YUVY.w = ((InputYUV4228bitBuffer[BufferIndex].DWord0 >> 24) & 0xFF) * Mul;

	int DoSRGBToLinear = 0;

	for (uint PixelIndex = 0; PixelIndex < PixelContainerSize; PixelIndex++)
	{
		const float3 RGB00 = YuvToRgb(PixelData[PixelIndex].YUVY.xyz, ColorTransform, DoSRGBToLinear);
		const float3 RGB01 = YuvToRgb(PixelData[PixelIndex].YUVY.wyz, ColorTransform, DoSRGBToLinear);
		OutTexture[PixelData[PixelIndex].UV1] = float4(RGB00, 1);
		OutTexture[PixelData[PixelIndex].UV2] = float4(RGB01, 1);
	}
}

// Shader to convert from YUV 422 10 bits packed little endian to RGBA

[numthreads(DISPATCH_GROUP_COUNT)]
void YUV10Bit422ToRGBACS(uint3 ThreadId : SV_DispatchThreadID, uint GroupIndex : SV_GroupIndex)
{
	// Make sure it's a valid thread
	if(ThreadId.x >= TotalElementCount)
	{
		return;
	}

	// Pixels we treat in shader is pretty much hardcoded to 8 so no parameter has been used
	const uint OutputTexturePixelsPerThread = 8;

	struct FOutputPixelData
	{
		uint2 UV1;
		uint2 UV2;
		float4 YUVY;
	};
	const uint PixelContainerSize = OutputTexturePixelsPerThread / 2;
	FOutputPixelData PixelData[PixelContainerSize];

	const uint SourcePixelStart = ThreadId.x * OutputTexturePixelsPerThread;
	for (uint ArrayIndex = 0; ArrayIndex < PixelContainerSize; ArrayIndex++)
	{
		const uint PixelIndex = ArrayIndex * 2;
		PixelData[ArrayIndex].UV1.x = ((SourcePixelStart + (PixelIndex + 0)) % OutputTextureSizeX); 
		PixelData[ArrayIndex].UV1.y = ((SourcePixelStart + (PixelIndex + 0)) / OutputTextureSizeX);
		PixelData[ArrayIndex].UV2.x = ((SourcePixelStart + (PixelIndex + 1)) % OutputTextureSizeX); 
		PixelData[ArrayIndex].UV2.y = ((SourcePixelStart + (PixelIndex + 1)) / OutputTextureSizeX);

		PixelData[ArrayIndex].YUVY = float4(0,0,0,0);
	}

	// Input buffer indexing is simply thread ix
	const uint BufferIndex = ThreadId.x;

	// Used to normalize pixel values based on 10 bits input
	const float Mul = 1.0 / 1023.0;

	PixelData[0].YUVY.x = ((((InputYCbCrBuffer[BufferIndex].DWord0 >>  8)  << 4) | ((InputYCbCrBuffer[BufferIndex].DWord0 >> 20) & 0x00F)) & 0x3FF) * Mul;
	PixelData[0].YUVY.y = ((((InputYCbCrBuffer[BufferIndex].DWord0 >>  0)  << 2) | ((InputYCbCrBuffer[BufferIndex].DWord0 >> 14) & 0x003)) & 0x3FF) * Mul;
	PixelData[0].YUVY.z = ((((InputYCbCrBuffer[BufferIndex].DWord0 >> 16)  << 6) | ((InputYCbCrBuffer[BufferIndex].DWord0 >> 26) & 0x03F)) & 0x3FF) * Mul;
	PixelData[0].YUVY.w = ((((InputYCbCrBuffer[BufferIndex].DWord0 >> 24)  << 8) | ((InputYCbCrBuffer[BufferIndex].DWord1 >> 00) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[1].YUVY.x = ((((InputYCbCrBuffer[BufferIndex].DWord1 >> 16)  << 4) | ((InputYCbCrBuffer[BufferIndex].DWord1 >> 28) & 0x00F)) & 0x3FF) * Mul;
	PixelData[1].YUVY.y = ((((InputYCbCrBuffer[BufferIndex].DWord1 >>  8)  << 2) | ((InputYCbCrBuffer[BufferIndex].DWord1 >> 22) & 0x003)) & 0x3FF) * Mul;
	PixelData[1].YUVY.z = ((((InputYCbCrBuffer[BufferIndex].DWord1 >> 24)  << 6) | ((InputYCbCrBuffer[BufferIndex].DWord2 >> 02) & 0x03F)) & 0x3FF) * Mul;
	PixelData[1].YUVY.w = ((((InputYCbCrBuffer[BufferIndex].DWord2 >> 00)  << 8) | ((InputYCbCrBuffer[BufferIndex].DWord2 >>  8) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[2].YUVY.x = ((((InputYCbCrBuffer[BufferIndex].DWord2 >> 24)  << 4) | ((InputYCbCrBuffer[BufferIndex].DWord3 >> 04) & 0x00F)) & 0x3FF) * Mul;
	PixelData[2].YUVY.y = ((((InputYCbCrBuffer[BufferIndex].DWord2 >> 16)  << 2) | ((InputYCbCrBuffer[BufferIndex].DWord2 >> 30) & 0x003)) & 0x3FF) * Mul;
	PixelData[2].YUVY.z = ((((InputYCbCrBuffer[BufferIndex].DWord3 >> 00)  << 6) | ((InputYCbCrBuffer[BufferIndex].DWord3 >> 10) & 0x03F)) & 0x3FF) * Mul;
	PixelData[2].YUVY.w = ((((InputYCbCrBuffer[BufferIndex].DWord3 >>  8)  << 8) | ((InputYCbCrBuffer[BufferIndex].DWord3 >> 16) & 0x0FF)) & 0x3FF) * Mul;
	PixelData[3].YUVY.x = ((((InputYCbCrBuffer[BufferIndex].DWord4 >>  0)  << 4) | ((InputYCbCrBuffer[BufferIndex].DWord4 >> 12) & 0x00F)) & 0x3FF) * Mul;
	PixelData[3].YUVY.y = ((((InputYCbCrBuffer[BufferIndex].DWord3 >> 24)  << 2) | ((InputYCbCrBuffer[BufferIndex].DWord4 >> 06) & 0x003)) & 0x3FF) * Mul;
	PixelData[3].YUVY.z = ((((InputYCbCrBuffer[BufferIndex].DWord4 >>  8)  << 6) | ((InputYCbCrBuffer[BufferIndex].DWord4 >> 18) & 0x03F)) & 0x3FF) * Mul;
	PixelData[3].YUVY.w = ((((InputYCbCrBuffer[BufferIndex].DWord4 >> 16)  << 8) | ((InputYCbCrBuffer[BufferIndex].DWord4 >> 24) & 0x0FF)) & 0x3FF) * Mul;


	int DoSRGBToLinear = 0;

	for (uint PixelIndex = 0; PixelIndex < PixelContainerSize; PixelIndex++)
	{
		const float3 RGB00 = YuvToRgb(PixelData[PixelIndex].YUVY.xyz, ColorTransform, DoSRGBToLinear);
		const float3 RGB01 = YuvToRgb(PixelData[PixelIndex].YUVY.wyz, ColorTransform, DoSRGBToLinear);
		OutTexture[PixelData[PixelIndex].UV1] = float4(RGB00, 1);
		OutTexture[PixelData[PixelIndex].UV2] = float4(RGB01, 1);
	}
}
