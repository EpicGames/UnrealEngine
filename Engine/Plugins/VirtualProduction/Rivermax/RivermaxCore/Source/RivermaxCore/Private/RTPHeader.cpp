// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTPHeader.h"

#include "RivermaxLog.h"

namespace UE::RivermaxCore::Private
{
	uint16 FRawSRD::GetRowNumber() const
	{
		return ((RowNumberHigh << 8) | RowNumberLow);
	}

	void FRawSRD::SetRowNumber(uint16 RowNumber)
	{
		RowNumberHigh = (RowNumber >> 8) & 0xFF;
		RowNumberLow = RowNumber & 0xFF;
	}

	uint16 FRawSRD::GetOffset() const
	{
		return ((OffsetHigh << 8) | OffsetLow);
	}

	void FRawSRD::SetOffset(uint16 Offset)
	{
		OffsetHigh = (Offset >> 8) & 0xFF;
		OffsetLow = Offset & 0xFF;
	}

	const uint8* GetRTPHeaderPointerVideo(const uint8* InHeader)
	{
		check(InHeader);

		static constexpr uint32 ETH_TYPE_802_1Q = 0x8100;          /* 802.1Q VLAN Extended Header  */
		static constexpr uint32 ETHERTYPE_OFFSET = 12;
		uint16 ETHProto = 0;

		FMemory::Memcpy(&ETHProto, InHeader + ETHERTYPE_OFFSET, sizeof(ETHProto));

#if PLATFORM_LITTLE_ENDIAN
		ETHProto = ByteSwap(ETHProto);
#endif

		// Our video SRDs are either 20 or 24 octets. This won't work for any other packet type.
		if (ETH_TYPE_802_1Q == ETHProto)
		{
			InHeader += 46; // 802 + 802.1Q + IP + UDP
		}
		else
		{
			InHeader += 42; // 802 + IP + UDP
		}
		return InHeader;
	}

	FRTPHeader::FRTPHeader(const FVideoRTPHeader& VideoRTP)
	{
		Timestamp = 0;

		if (VideoRTP.RTPHeader.Version != 2)
		{
			return;
		}

		// Pretty sure some data needs to be swapped but can't validate that until we have other hardware generating data
		SequenceNumber = (ByteSwap((uint16)VideoRTP.RTPHeader.ExtendedSequenceNumber) << 16) | ByteSwap((uint16)VideoRTP.RTPHeader.SequenceNumber);
		Timestamp = ByteSwap(VideoRTP.RTPHeader.Timestamp);
		bIsMarkerBit = VideoRTP.RTPHeader.MarkerBit;

		SyncSouceId = VideoRTP.RTPHeader.SynchronizationSource;
#if PLATFORM_LITTLE_ENDIAN
		SyncSouceId = ByteSwap(SyncSouceId);
#endif

		SRD1.Length = ByteSwap((uint16)VideoRTP.SRD1.Length);
		SRD1.DataOffset = VideoRTP.SRD1.GetOffset();
		SRD1.RowNumber = VideoRTP.SRD1.GetRowNumber();
		SRD1.bIsFieldOne = VideoRTP.SRD1.FieldIdentification;
		SRD1.bHasContinuation = VideoRTP.SRD1.ContinuationBit;

		if (SRD1.bHasContinuation)
		{
			SRD2.Length = ByteSwap((uint16)VideoRTP.SRD2.Length);
			SRD2.DataOffset = VideoRTP.SRD2.GetOffset();
			SRD2.RowNumber = VideoRTP.SRD2.GetRowNumber();
			SRD2.bIsFieldOne = VideoRTP.SRD2.FieldIdentification;
			SRD2.bHasContinuation = VideoRTP.SRD2.ContinuationBit;

			if (SRD2.bHasContinuation == true)
			{
				UE_LOGF(LogRivermax, Verbose, "Received SRD with more than 2 SRD which isn't supported.");
			}
		}
	}

	uint16 FRTPHeader::GetTotalPayloadSize() const
	{
		uint16 PayloadSize = SRD1.Length;
		if (SRD1.bHasContinuation)
		{
			PayloadSize += SRD2.Length;
		}

		return PayloadSize;
	}

	uint16 FRTPHeader::GetLastPayloadSize() const
	{
		if (SRD1.bHasContinuation)
		{
			return SRD2.Length;
		}

		return SRD1.Length;
	}

	uint16 FRTPHeader::GetLastRowOffset() const
	{
		if (SRD1.bHasContinuation)
		{
			return SRD2.DataOffset;
		}

		return SRD1.DataOffset;
	}

	uint16 FRTPHeader::GetLastRowNumber() const
	{
		if (SRD1.bHasContinuation)
		{
			return SRD2.RowNumber;
		}

		return SRD1.RowNumber;
	}
}






