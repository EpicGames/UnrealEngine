// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"

#if UE_BUILD_DEBUG
	#define RIVERMAX_PACKET_DEBUG 1
#else
	#define RIVERMAX_PACKET_DEBUG 0
#endif

#if RIVERMAX_PACKET_DEBUG
	#define RIVERMAX_DEBUG_FIELD_NAME(...) , __VA_ARGS__
#else
	#define RIVERMAX_DEBUG_FIELD_NAME(...)
#endif


namespace UE::RivermaxCore::Private
{
	//RTP Header used for 2110 following https://www.rfc-editor.org/rfc/rfc4175.html

	/* RTP Header -  14 bytes
	0                   1                   2                   3
	0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	| V |P|X|  CC   |M|     PT      |            SEQ                |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	|                           timestamp                           |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	|                           ssrc                                |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	|    Extended Sequence Number   |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	*/

	/** Raw representation as it's built for the network */

/** @note When other platform than windows are supports, reverify support for pragma_pack and endianness */
#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(push, 1)
#endif
	/** Total size should be 14 octets. */
	struct FRawRTPHeader
	{
		uint8 ContributingSourceCount   : 4;
		uint8 ExtensionBit              : 1;
		uint8 PaddingBit                : 1;
		uint8 Version                   : 2;
		uint8 PayloadType               : 7;
		uint8 MarkerBit                 : 1;
		uint16 SequenceNumber           : 16;
		uint32 Timestamp                : 32;
		uint32 SynchronizationSource    : 32;
		uint16 ExtendedSequenceNumber   : 16;

		/** The base value is 0x0eb51dbf which will be used for video. Anc is VideoSyncSource + 1. */
		static constexpr uint32 VideoSynchronizationSource = 0x0eb51dbf;
	};

	/** 
	SRD Header. Total packed size should be 6 octets.

	0                   1                   2                   3
	0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	|           SRD Length          |F|     SRD Row Number          |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	|C|         SRD Offset          |
	+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	*/
	struct FRawSRD
	{
		uint16 Length : 16;

		uint8 RowNumberHigh : 7;

		uint8 FieldIdentification : 1;

		uint8 RowNumberLow : 8;

		uint8 OffsetHigh : 7;

		/** If set indicates that there is another SRD following this one.*/
		uint8 ContinuationBit : 1;
		uint8 OffsetLow : 8;


		/** Returns SRD associated row number */
		uint16 GetRowNumber() const;

		/** Sets SRD associated row number */
		void SetRowNumber(uint16 RowNumber);

		/** Returns SRD pixel offset in its associated row */
		uint16 GetOffset() const;

		/** Sets SRD pixel offset in its associated row */
		void SetOffset(uint16 Offset);
	};

	/** Total size should be 26 octets. */
	struct FVideoRTPHeader
	{
		FRawRTPHeader RTPHeader;

		FRawSRD SRD1;                   // 48

		FRawSRD SRD2;                   // 48

		/** Size of RTP representation whether it has one or two SRDs */
		static constexpr uint32 OneSRDSize = 20;
		static constexpr uint32 TwoSRDSize = 26;
	};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
#pragma pack(pop)
#endif

	struct FSRDHeader
	{
		/** Length of payload. Is a multiple of pgroup (see pixel formats) */
		uint16 Length = 0;

		/** False if progressive or first field of interlace. True if second field of interlace */
		bool bIsFieldOne = false;

		/** Video line number, starts at 0 */
		uint16 RowNumber = 0;

		/** Whether another SRD is following this one */
		bool bHasContinuation = false;

		/** Location of the first pixel in payload, in pixel */
		uint16 DataOffset = 0;
	};

	/** RTP header built from network representation not requiring any byte swapping */
	struct FRTPHeader
	{
		FRTPHeader() = default;
		FRTPHeader(const FVideoRTPHeader& VideoRTP);

		/** Returns the total payload of this RTP */
		uint16 GetTotalPayloadSize() const;
		
		/** Returns the payload size of the last SRD in this RTP */
		uint16 GetLastPayloadSize() const;

		/** Returns the row offset of the last SRD in this RTP */
		uint16 GetLastRowOffset() const;

		/** Returns the row number of the last SRD in this RTP */
		uint16 GetLastRowNumber() const;

		/** Sequence number including extension if present */
		uint32 SequenceNumber = 0;

		/** Timestamp of frame in the specified clock resolution. Video is typically 90kHz */
		uint32 Timestamp = 0;

		/** Identification of this stream */
		uint32 SyncSouceId = 0;

		/** Whether extensions (SRD headers) are present */
		bool bHasExtension = false;

		/** True if RTP packet is last of video stream */
		bool bIsMarkerBit = false;

		/** Only supports 2 SRD for now. Adjust if needed */
		FSRDHeader SRD1;
		FSRDHeader SRD2;
	};

	/** 
	* Returns RTPHeader pointer from a raw ethernet packet skipping 802, IP, UDP headers.
	* This only works for IPv4 2110-20 video single SRD or Two-srd RTP packets.
	* Will not work correctly for audio, ANC, or non-standard RTP packets.
	* Assumes InHeader has at least 14 bytes; no bounds checking is performed.
	*/
	const uint8* GetRTPHeaderPointerVideo(const uint8* InHeader);


	/** 
	* A class that helps with packet creation and endianness conversion.
	* The main purpose of this is to first pack bits, second is to handle endianness.
	*/
	class FBigEndianHeaderPacker
	{
	private:
		/** All bytes containing fields in big endian ready for network. */
		TArrayView<uint8> Data;

		/** Counting elements in the array. */
		int32 CurrentByte;

		/** Bits used in the current byte. */
		int32 RemainingBitsInCurrentByte = 0;

		/** Once packet is finalized certain actions are no longer possible. */
		bool bFinalized = false;


#if RIVERMAX_PACKET_DEBUG
	public:
		/**
		* Debug struct that is used for storing field information.
		*/
		struct FFieldInfo
		{
			union
			{
				uint32 Data;
				int32 BitPosition;
			};
			int32 NumBits;
			FString FieldName;
		};

		TArray<FFieldInfo> Fields;


	private:

		/** 
		* Used only in debug mode. These are processed field bits as they appear in the packet. 
		* Can be accessed to output into an external file or general debugging.
		*/
		TArray<TArray<FFieldInfo>> FieldsInOrder;
#endif

	public:

		/** 
		* Users responsible for specifying a vaild memory address for data to be packed into with enough memory.
		* If existing data must be cleared bClearExisting must be set to true.
		*/
		FBigEndianHeaderPacker(uint8* BufferToFill, int32 MaxSize, bool bClearExisting = true)
			: Data(BufferToFill, MaxSize)
			, CurrentByte(0)
		{
			check(MaxSize > 0);
			if (bClearExisting)
			{
				// making sure that packet is zeroed out.
				memset(BufferToFill, 0, MaxSize);
			}

			RemainingBitsInCurrentByte = 8;

#if RIVERMAX_PACKET_DEBUG
			FieldsInOrder.SetNum(MaxSize);
#endif

		}

		/**
		* Adds a field in order. Non-debug version immediately packs it and handles endiannes and then writes into the byte array. 
		* Debug version keeps fields in a user readable format to be packed later.  
		*/
#if RIVERMAX_PACKET_DEBUG
		FORCEINLINE void AddField(uint32 InFieldData, int32 InNumBits, const FString& DebugString)
		{
			FFieldInfo Field;
			Field.Data = InFieldData;
			Field.NumBits = InNumBits;
			Field.FieldName = DebugString;
			Fields.Add(MoveTemp(Field));
		}
#else
		FORCEINLINE void AddField(uint32 InFieldData, int32 InNumBits)
		{
			AddFieldInternal(InFieldData, InNumBits, false /*bClear*/);
		}
#endif

		/** 
		* Function that updates a field in an already formed existing packet. Useful when re-using packets from the previous frame.
		*/
#if RIVERMAX_PACKET_DEBUG
		FORCEINLINE void UpdateField(uint32 InFieldData, int32 InNumBits, int32 FieldPositionInBits, const FString& DebugString)
#else
		FORCEINLINE void UpdateField(uint32 InFieldData, int32 InNumBits, int32 FieldPositionInBits)
#endif
		{
			// Not allowed to update if the packet isn't finalized because it will ruin the order
			check(bFinalized);

			constexpr int32 SizeOfAByteInBits = 8;

			int32 Offset = FieldPositionInBits / SizeOfAByteInBits;
			int32 PositionInByte = (FieldPositionInBits % SizeOfAByteInBits);
			RemainingBitsInCurrentByte = (PositionInByte == 0) ? SizeOfAByteInBits : SizeOfAByteInBits - PositionInByte;

			check(FieldPositionInBits + InNumBits <= Data.Num() * 8);
			CurrentByte = Offset;
			constexpr bool bClearExisting = true;
#if RIVERMAX_PACKET_DEBUG
			AddFieldInternal(InFieldData, InNumBits, bClearExisting, DebugString);
#else
			AddFieldInternal(InFieldData, InNumBits, bClearExisting);
#endif
		}


		/* 
		* WordAlign - the word size in bytes to be aligned to. For example Anc packets need to be aligned to the nearest 32 bit word
		*/
		FORCEINLINE void WordAlign(const int32 InWorldAlign)
		{
			check(InWorldAlign > 0);
			constexpr int32 SizeOfAByteInBits = 8;
			int32 NumBits = CountFieldBits();
			const int32 WordBitsToAlignTo = (InWorldAlign * SizeOfAByteInBits);
			int32 BitMisalignment = NumBits % WordBitsToAlignTo;
			int32 BitsToAdd = (BitMisalignment == 0) ? 0 : WordBitsToAlignTo - BitMisalignment;

			if (BitsToAdd > 0)
			{
#if RIVERMAX_PACKET_DEBUG
				AddField(0, BitsToAdd, "WordAlign");
#else
				AddField(0, BitsToAdd);
#endif
			}
		}


		/** 
		* Cleanup and pad the memory if it is necessary. If packet debugging is enabled, convert it into proper packet. 
		*/
		FORCEINLINE void Finalize()
		{
			check(!bFinalized);

			bFinalized = true;

#if RIVERMAX_PACKET_DEBUG
			for (FFieldInfo& Field : Fields)
			{
				AddFieldInternal(Field.Data, Field.NumBits, false, Field.FieldName);
			}
#endif
			
			constexpr int32 SizeOfAByteInBits = 8;

			if (CurrentByte > 0 && RemainingBitsInCurrentByte == SizeOfAByteInBits)
			{
				CurrentByte--;
				RemainingBitsInCurrentByte = 0;
			}

		}

		FORCEINLINE int32 CountFieldBits()
		{
			int32 TotalBits = 0;
#if RIVERMAX_PACKET_DEBUG
			for (FFieldInfo& Field : Fields)
			{
				TotalBits += Field.NumBits;
			}
#else
			TotalBits = ((CurrentByte + 1) * 8 - RemainingBitsInCurrentByte);
#endif
			return TotalBits;
		}

		/** Size in bytes. */
		FORCEINLINE int64 CountBytes()
		{
			constexpr int32 SizeOfAByteInBits = 8;
			return (CountFieldBits() + SizeOfAByteInBits - 1) / SizeOfAByteInBits;
		}

#if RIVERMAX_PACKET_DEBUG
		const TArray<TArray<FFieldInfo>>& GetFieldsInOrderByRef()
		{
			check(bFinalized);
			return FieldsInOrder;
		}
#endif

	private:

#if RIVERMAX_PACKET_DEBUG
		void AddFieldInternal(uint32 InFieldData, int32 InNumBits, bool bClear, const FString& DebugString)
#else
		FORCEINLINE void AddFieldInternal(uint32 InFieldData, int32 InNumBits, bool bClear)
#endif
		{
			check(InNumBits > 0 && InNumBits <= 32);
			constexpr int32 SizeOfAByteInBits = 8;
			int32 BitsLeftToPack = InNumBits;


			while (BitsLeftToPack > 0)
			{
				check(CurrentByte < Data.Num());

#if RIVERMAX_PACKET_DEBUG
				uint8* CurrentBytePtr = &Data[CurrentByte];
#else
				// No range check.
				uint8* CurrentBytePtr = &Data.GetData()[CurrentByte];
#endif
				// RemainingBitsInCurrentByte is guaranteed to be <= 8.
				int32 CurrentBitsToPack = FMath::Min(BitsLeftToPack, RemainingBitsInCurrentByte);
				BitsLeftToPack -= CurrentBitsToPack;

				// CurrentBitsToPack doesn't exceed RemainingBitsInCurrentByte, which is <= 8.
				uint32 Mask = (1u << CurrentBitsToPack) - 1;

#if PLATFORM_LITTLE_ENDIAN
				// Pack higher order bits first on little endian.
				uint32 BitsToWrite = (InFieldData >> BitsLeftToPack) & Mask;
#else
				uint32 BitsToWrite = InFieldData & Mask;
				InFieldData >>= CurrentBitsToPack;
#endif

				int32 BitPosition = RemainingBitsInCurrentByte - CurrentBitsToPack;
				BitsToWrite <<= BitPosition;

				if (bClear)
				{
					uint8 ClearMask = static_cast<uint8>(Mask) << BitPosition;
					*CurrentBytePtr &= ~ClearMask;
				}

				*CurrentBytePtr |= BitsToWrite;


#if RIVERMAX_PACKET_DEBUG
				FFieldInfo Field;
				Field.BitPosition = BitPosition;
				Field.NumBits = CurrentBitsToPack;
				Field.FieldName = DebugString;
				FieldsInOrder[CurrentByte].Add(MoveTemp(Field));
#endif

				RemainingBitsInCurrentByte -= CurrentBitsToPack;
				if (RemainingBitsInCurrentByte == 0)
				{
					RemainingBitsInCurrentByte = SizeOfAByteInBits;
					CurrentByte++;
				}


			}
		}

	};

	/**
	 * Counterpart to FBigEndianHeaderPacker. Reads fields from a big-endian Most significant byte first packed
	 */
	class FBigEndianHeaderUnpacker
	{
	public:
		FBigEndianHeaderUnpacker(const uint8* InBuffer, int32 InMaxSize)
			: Data(InBuffer, InMaxSize)
		{
			check(InMaxSize > 0);
		}

		FORCEINLINE uint32 GetNextField(int32 InNumBits)
		{
			check(InNumBits > 0 && InNumBits <= 32);
			uint32 Result = 0;
			const int32 EndBit = CurrentBit + InNumBits;
			for (int32 BitIdx = CurrentBit; BitIdx < EndBit; ++BitIdx)
			{
				const int32 ByteIdx   = BitIdx / 8;
				const int32 BitInByte = 7 - (BitIdx % 8); // MSB-first within each byte
				Result = (Result << 1) | ((Data[ByteIdx] >> BitInByte) & 1u);
			}
			CurrentBit = EndBit;
			return Result;
		}

		/** Skip InNumBits without reading. */
		FORCEINLINE void SkipBits(int32 InNumBits)
		{
			check(InNumBits >= 0);
			CurrentBit += InNumBits;
		}

		/**
		 * Advance to the next InWordSizeByte boundary.
		 */
		FORCEINLINE void WordAlign(int32 InWordSizeByte)
		{
			check(InWordSizeByte > 0);
			const int32 WordBits     = InWordSizeByte * 8;
			const int32 Misalignment = CurrentBit % WordBits;
			if (Misalignment != 0)
			{
				CurrentBit += WordBits - Misalignment;
			}
		}

		/** Returns true if at least InNumBits remain in the buffer. */
		FORCEINLINE bool CanRead(int32 InNumBits) const
		{
			return CurrentBit + InNumBits <= Data.Num() * 8;
		}

		/** Current bit cursor position. */
		FORCEINLINE int32 GetBitPosition() const { return CurrentBit; }

	private:
		TArrayView<const uint8> Data;
		int32 CurrentBit = 0;
	};

}


