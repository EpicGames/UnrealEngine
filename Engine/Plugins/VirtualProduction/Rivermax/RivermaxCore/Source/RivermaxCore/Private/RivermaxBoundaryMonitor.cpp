// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxBoundaryMonitor.h"

#include "HAL/IConsoleManager.h"
#include "HAL/RunnableThread.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RivermaxPTPUtils.h"

namespace UE::RivermaxCore
{
	static TMap<FFrameRate, FGuid> GBoundaryListeners;

	/** 
	 * Parses frame rate from a string 
	 * Can either be numerator only, assuming denominator of 1 (e.g. 24)
	 * Or can be explicit numerator over denominator (e.g. 24000/1001)
	 */
	static FFrameRate ParseFrameRate(const FString& Command)
	{
		int32 Numerator = 1;
		int32 Denominator = 1;
		int32 DividerIndex = INDEX_NONE;
		Command.FindChar('/', DividerIndex);
		if (DividerIndex != INDEX_NONE)
		{
			Numerator = FCString::Atoi(*Command.Left(DividerIndex));
			Denominator = FCString::Atoi(*Command.Right(DividerIndex));
		}
		else
		{
			Numerator = FCString::Atoi(*Command);
		}

		return FFrameRate(Numerator, Denominator);
	}

	static FAutoConsoleCommand CRivermaxBoundaryMonitorAddFrameRate(
		TEXT("Rivermax.Monitor.Add"),
		TEXT("Add monitoring for a certain frame rate.")
		TEXT("Example value format: \"Rivermax.Monitor.Add 24\"")
		TEXT("Example value format: \"Rivermax.Monitor.Add 60000/1001\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					const FFrameRate DesiredRate = ParseFrameRate(Args[0]);
					if (DesiredRate.IsValid() && DesiredRate.Numerator > 0)
					{
						if (!GBoundaryListeners.Contains(DesiredRate))
						{
							IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
							GBoundaryListeners.FindOrAdd(DesiredRate) = RivermaxModule.GetRivermaxBoundaryMonitor().StartMonitoring(DesiredRate);
						}
					}
				}
			}),
		ECVF_Default);

	static FAutoConsoleCommand CRivermaxBoundaryMonitorRemoveFrameRate(
		TEXT("Rivermax.Monitor.Remove"),
		TEXT("Add monitoring for a certain frame rate.")
		TEXT("Example value format: \"Rivermax.Monitor.Remove 24\"")
		TEXT("Example value format: \"Rivermax.Monitor.Remove 60000/1001\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num() > 0)
				{
					const FFrameRate DesiredRate = ParseFrameRate(Args[0]);
					if (DesiredRate.IsValid() && DesiredRate.Numerator > 0)
					{
						if (GBoundaryListeners.Contains(DesiredRate))
						{
							IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
							RivermaxModule.GetRivermaxBoundaryMonitor().StopMonitoring(GBoundaryListeners[DesiredRate], DesiredRate);
							GBoundaryListeners.Remove(DesiredRate);
						}
					}
				}
			}),
		ECVF_Default);

	FBoundaryMonitor::FBoundaryMonitor(const FFrameRate& InRate)
		: Rate(InRate)
	{
		IRivermaxCoreModule& RivermaxModule = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
		RivermaxManager = RivermaxModule.GetRivermaxManager();

		bIsEnabled = true;

		const FString ThreadName = FString::Printf(TEXT("Rmax_Boundary_%d/%d"), Rate.Numerator, Rate.Denominator);
		constexpr uint32 StackSize = 32 * 1024;
		WorkingThread.Reset(FRunnableThread::Create(this, *ThreadName, StackSize, TPri_AboveNormal));
	}

	FBoundaryMonitor::~FBoundaryMonitor()
	{
		bIsEnabled = false;
		WorkingThread->Kill();
	}

	uint32 FBoundaryMonitor::Run()
	{
		uint64 LastFrameNumber = UE::RivermaxCore::GetFrameNumber(RivermaxManager->GetTime(), Rate);
		while (bIsEnabled)
		{
			const uint64 NextFrameNumber = LastFrameNumber + 1;
			const uint64 TargetTimeNanosec = UE::RivermaxCore::GetAlignmentPointFromFrameNumber(NextFrameNumber, Rate);
			const uint64 CurrentPTPTime = RivermaxManager->GetTime();
			const double CurrentPlatformTime = FPlatformTime::Seconds();

			double TimeLeftNanosec = 0.0;
			if (TargetTimeNanosec >= CurrentPTPTime)
			{
				TimeLeftNanosec = TargetTimeNanosec - CurrentPTPTime;
			}

			const double WaitTimeSec = FMath::Clamp(TimeLeftNanosec / 1E9, 0.0, 5.0);
			const double StartTime = FPlatformTime::Seconds();
			double ActualWaitTime = 0.0;
			{
				static constexpr float SleepThresholdSec = 5.0f * (1.0f / 1000.0f);
				static constexpr float SpinningTimeSec = 2.0f * (1.0f / 1000.0f);
				if (WaitTimeSec > SleepThresholdSec)
				{
					FPlatformProcess::SleepNoStats(WaitTimeSec - SpinningTimeSec);
				}

				while (FPlatformTime::Seconds() < (CurrentPlatformTime + WaitTimeSec))
				{
					FPlatformProcess::SleepNoStats(0.f);
				}
			}

			TRACE_BOOKMARK(TEXT("%d/%d: Frame %llu"), Rate.Numerator, Rate.Denominator, NextFrameNumber);
			LastFrameNumber = NextFrameNumber;
		}

		return 0;
	}

FRivermaxBoundaryMonitor::~FRivermaxBoundaryMonitor()
{
	constexpr bool bEnableMonitoring = false;
	EnableMonitoring(bEnableMonitoring);
}

void FRivermaxBoundaryMonitor::EnableMonitoring(bool bEnable)
{
	IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
	if(RivermaxModule && RivermaxModule->GetRivermaxManager()->IsLibraryInitialized())
	{
		if (bEnable != bIsEnabled)
		{
			bIsEnabled = bEnable;

			for (const TTuple<FFrameRate, TArray<FGuid>>& Listener : ListenersMap)
			{
				if (bIsEnabled)
				{
					StartMonitor(Listener.Key);
				}
				else
				{
					StopMonitor(Listener.Key);
				}
			}
		}
	}
}

FGuid FRivermaxBoundaryMonitor::StartMonitoring(const FFrameRate& FrameRate)
{
	const FGuid NewListener = FGuid::NewGuid();
	ListenersMap.FindOrAdd(FrameRate).Add(NewListener);

	if (bIsEnabled)
	{
		StartMonitor(FrameRate);
	}

	return NewListener;
}

void FRivermaxBoundaryMonitor::StopMonitoring(const FGuid& Requester, const FFrameRate& FrameRate)
{
	bool bNoMoreListeners = false;
	if(TArray<FGuid>* Listeners = ListenersMap.Find(FrameRate))
	{
		Listeners->RemoveSingleSwap(Requester);
		bNoMoreListeners = Listeners->Num() <= 0;
	}

	if (bNoMoreListeners)
	{
		ListenersMap.Remove(FrameRate);
		StopMonitor(FrameRate);
	}
}

void FRivermaxBoundaryMonitor::StartMonitor(const FFrameRate& FrameRate)
{
	if (!MonitorMap.Contains(FrameRate))
	{
		MonitorMap.Add(FrameRate, MakeUnique<FBoundaryMonitor>(FrameRate));
	}
}

void FRivermaxBoundaryMonitor::StopMonitor(const FFrameRate& FrameRate)
{
	TUniquePtr<FBoundaryMonitor>* Monitor = MonitorMap.Find(FrameRate);
	if (Monitor)
	{
		(*Monitor).Reset();
	}

	MonitorMap.Remove(FrameRate);
}

}
