// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxBoundaryMonitor.h"

#include "HAL/Runnable.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"

namespace UE::RivermaxCore
{

class IRivermaxManager;

/** Monitor running its own thread to add trace bookmarks at frame boundary for a given frame rate */
class FBoundaryMonitor : public FRunnable
{
public:
	FBoundaryMonitor(const FFrameRate& Rate);
	virtual ~FBoundaryMonitor();

	//~ Begin FRunnable interface
	virtual uint32 Run() override;
	//~ End FRunnable interface

protected:

	/** Thread responsible to watch for frame boundaries */
	TUniquePtr<FRunnableThread> WorkingThread;

	/** Frame rate at which we add markups */
	FFrameRate Rate;

	/** Cached pointer to rivermax manager */
	TSharedPtr<IRivermaxManager> RivermaxManager;

	/** Whether thread should be running */
	std::atomic<bool> bIsEnabled = false;
};

/**
 * Monitoring manager of frame boundary based on ST-2059 / PTP formulae
 * Used to add trace bookmarks at each boundary for any frame rate monitored
 */
class FRivermaxBoundaryMonitor : public IRivermaxBoundaryMonitor
{
public:
	virtual ~FRivermaxBoundaryMonitor();

public:
	//~ Begin IRivermaxBoundaryMonitor interface
	virtual void EnableMonitoring(bool bEnable) override;
	virtual FGuid StartMonitoring(const FFrameRate& FrameRate) override;
	virtual void StopMonitoring(const FGuid& Requester, const FFrameRate& FrameRate) override;
	//~ End IRivermaxBoundaryMonitor interface

protected:

	/** Starts monitoring a given frame rate */
	void StartMonitor(const FFrameRate& FrameRate);
	
	/** Stops monitoring a given frame rate */
	void StopMonitor(const FFrameRate& FrameRate);

private:

	/** Whether frame boundary monitor is enabled */
	bool bIsEnabled = false;

	/** Maps of monitor to a given frame rate */
	TMap<FFrameRate, TUniquePtr<FBoundaryMonitor>> MonitorMap;

	/** Maps from a given frame rate to listeners ID */
	TMap<FFrameRate, TArray<FGuid>> ListenersMap;
};

}
