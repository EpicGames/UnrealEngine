// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxCoreModule.h"

#include "HAL/IConsoleManager.h"
#include "RivermaxLog.h"
#include "Modules/ModuleManager.h"
#include "RivermaxBoundaryMonitor.h"
#include "RivermaxFrameManager.h"
#include "Streams/RivermaxInputVideoStream.h"
#include "Streams/RivermaxInAncStream.h"
#include "RivermaxManager.h"
#include "Streams/RivermaxOutVideoStream.h"
#include "Streams/RivermaxOutAncStream.h"


static TAutoConsoleVariable<int32> CVarRivermaxEnableBoundaryMonitor(
	TEXT("Rivermax.Monitor.Enable"), 1,
	TEXT("Whether frame boundary monitor is enabled."),
	ECVF_Default);

void FRivermaxCoreModule::StartupModule()
{
	RivermaxManager = MakeShared<UE::RivermaxCore::Private::FRivermaxManager>();
	BoundaryMonitor = MakeUnique<UE::RivermaxCore::FRivermaxBoundaryMonitor>();
	RivermaxManager->OnPostRivermaxManagerInit().AddRaw(this, &FRivermaxCoreModule::OnRivermaxManagerInitialized);
}

void FRivermaxCoreModule::ShutdownModule()
{
	BoundaryMonitor->EnableMonitoring(false);
}

TUniquePtr<UE::RivermaxCore::IRivermaxInputStream> FRivermaxCoreModule::CreateInputStream(UE::RivermaxCore::ERivermaxStreamType StreamType, const TArray<char>& InSDPDescription)
{
	using namespace UE::RivermaxCore::Private;
	switch (StreamType)
	{
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_20:
		return MakeUnique<FRivermaxInputVideoStream>();
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_40:
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_40_TC:
		return MakeUnique<FRivermaxInputAncStream>();
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_30:
	default:
		UE_LOG(LogRivermax, Warning, TEXT("CreateInputStream: Stream type %d is not supported."), (int32)StreamType);
		return nullptr;
	}
}

TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream> FRivermaxCoreModule::CreateOutputStream(UE::RivermaxCore::ERivermaxStreamType StreamType, const TArray<char>& InSDPDescription)
{
	using namespace UE::RivermaxCore::Private;
	switch (StreamType)
	{
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_20:
		return MakeUnique<FRivermaxOutputStreamVideo>(InSDPDescription);
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_40_TC:
		return MakeUnique<FRivermaxOutputStreamAncTimecode>(InSDPDescription);
	case UE::RivermaxCore::ERivermaxStreamType::ST2110_30:
	default:
		return nullptr;
	}
}

TSharedPtr<UE::RivermaxCore::IRivermaxManager> FRivermaxCoreModule::GetRivermaxManager()
{
	return RivermaxManager;
}

UE::RivermaxCore::IRivermaxBoundaryMonitor& FRivermaxCoreModule::GetRivermaxBoundaryMonitor()
{
	return *BoundaryMonitor;
}

void FRivermaxCoreModule::OnRivermaxManagerInitialized()
{
	FConsoleVariableDelegate OnBoundaryMonitorEnableValueChanged = FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
	{
		IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
		if (RivermaxModule)
		{
			RivermaxModule->GetRivermaxBoundaryMonitor().EnableMonitoring(CVar->GetBool());
		}
	});
	CVarRivermaxEnableBoundaryMonitor.AsVariable()->SetOnChangedCallback(OnBoundaryMonitorEnableValueChanged);
	
	const bool bDoEnable = CVarRivermaxEnableBoundaryMonitor.GetValueOnGameThread() == 1;
	BoundaryMonitor->EnableMonitoring(bDoEnable);
}

IMPLEMENT_MODULE(FRivermaxCoreModule, RivermaxCore);
