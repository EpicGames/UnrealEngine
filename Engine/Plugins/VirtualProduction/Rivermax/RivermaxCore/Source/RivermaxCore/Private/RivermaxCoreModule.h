// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxCoreModule.h"

namespace UE::RivermaxCore
{
	class IRivermaxManager;
}

/**
 * Implementation of the IRivermaxCoreModule managing Rivermax library and stream creation
 */
class FRivermaxCoreModule : public IRivermaxCoreModule
{

public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IRivermaxCoreModule interface
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxInputStream> CreateInputStream(UE::RivermaxCore::ERivermaxStreamType, const TArray<char>& InSDPDescription) override;
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream> CreateOutputStream(UE::RivermaxCore::ERivermaxStreamType, const TArray<char>& InSDPDescription) override;
	virtual TSharedPtr<UE::RivermaxCore::IRivermaxManager> GetRivermaxManager() override;
	virtual UE::RivermaxCore::IRivermaxBoundaryMonitor& GetRivermaxBoundaryMonitor() override;
	//~ End IRivermaxCoreModule interface

private:

	/** Called when manager has finished initializing */
	void OnRivermaxManagerInitialized();

private:

	/** Manager handling library initialization */
	TSharedPtr<UE::RivermaxCore::IRivermaxManager> RivermaxManager;

	/** Boundary monitor used to add trace information for frame boundaries at different frame rates */
	TUniquePtr<UE::RivermaxCore::IRivermaxBoundaryMonitor> BoundaryMonitor;
};

