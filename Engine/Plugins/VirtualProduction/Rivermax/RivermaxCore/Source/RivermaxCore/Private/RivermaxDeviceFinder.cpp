// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxDeviceFinder.h"

#include "Misc/LazySingleton.h"
#include "IRivermaxCoreModule.h"
#include "RivermaxLog.h"


#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"

#include <winsock2.h>
#include <Iphlpapi.h>
#include <ws2tcpip.h>

#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_LINUX
#include <ifaddrs.h>
#include <net/if.h>
#include <arpa/inet.h>
#include <fcntl.h>
#include <unistd.h>
#endif


namespace UE::RivermaxCore::Private
{
	FRivermaxDeviceFinder::FRivermaxDeviceFinder()
	{
		FindDevices();
	}

	bool FRivermaxDeviceFinder::IsValidIP(const FString& SourceIP) const
	{
		const uint32 IPHash = GetTypeHash(SourceIP);
		if (FQueriedIPInfo* FoundQuery = QueriesCache.Find(IPHash))
		{
			return FoundQuery->bIsValidIP;
		}

		TArray<FString> SourceTokens;
		bool bIsValid = SourceIP.ParseIntoArray(SourceTokens, TEXT("."), false /*CullEmpty*/) == 4;
		if (bIsValid)
		{
			for (const FString& Token : SourceTokens)
			{
				if (!FCString::IsNumeric(*Token))
				{
					// If it's not a pure number, look for individual characters
					for (int32 Index = 0; Index < Token.Len(); ++Index)
					{
						const TCHAR& Character = Token[Index];
						if (!(FChar::IsDigit(Character) || Character == '*' || Character == '?'))
						{
							bIsValid = false;
							break;
						}
					}

					if (!bIsValid)
					{
						break;
					}
				}
			}
		}

		FQueriedIPInfo& FoundQuery = QueriesCache.FindOrAdd(IPHash);
		FoundQuery.bIsValidIP = bIsValid;
		FoundQuery.SourceIP = SourceIP;
		FoundQuery.Tokens = MoveTemp(SourceTokens);

		return bIsValid;
	}

	bool FRivermaxDeviceFinder::ResolveIP(const FString& SourceIP, FString& OutDeviceIP) const
	{
		if (IsValidIP(SourceIP))
		{
			const uint32 IPHash = GetTypeHash(SourceIP);
			FQueriedIPInfo& FoundQuery = QueriesCache[IPHash];
			if (FoundQuery.ResolvedDeviceIPHash.IsSet())
			{
				// Device cache should have this entry if the hash has been set in the IP cache
				OutDeviceIP = DevicesCache[FoundQuery.ResolvedDeviceIPHash.GetValue()].DeviceIP;
				return true;
			}

			for (const FRivermaxDeviceInfo& Device : Devices)
			{
				const uint32 DeviceIPHash = GetTypeHash(Device.InterfaceAddress);
				if (DevicesCache.Contains(DeviceIPHash) == false)
				{
					FDeviceInfo& NewDevice = DevicesCache.FindOrAdd(DeviceIPHash);
					NewDevice.DeviceIP = Device.InterfaceAddress;
					verify(FIPv4Address::Parse(NewDevice.DeviceIP, NewDevice.Address));
					NewDevice.AddressTokens[0] = NewDevice.Address.A;
					NewDevice.AddressTokens[1] = NewDevice.Address.B;
					NewDevice.AddressTokens[2] = NewDevice.Address.C;
					NewDevice.AddressTokens[3] = NewDevice.Address.D;

					// Also cache tokens as string
					NewDevice.DeviceIP.ParseIntoArray(NewDevice.Tokens, TEXT("."), false /*CullEmpty*/);
				}

				const FDeviceInfo& CachedDevice = DevicesCache[DeviceIPHash];

				bool bMatchingDeviceFound = true;
				for (int32 TokenIndex = 0; TokenIndex < 4; ++TokenIndex)
				{
					const FString& DesiredToken = FoundQuery.Tokens[TokenIndex];
					const FString& DeviceToken = CachedDevice.Tokens[TokenIndex];
					if (!DeviceToken.MatchesWildcard(DesiredToken))
					{
						bMatchingDeviceFound = false;
						break;
					}
				}

				if (bMatchingDeviceFound)
				{
					FoundQuery.ResolvedDeviceIPHash = DeviceIPHash;
					OutDeviceIP = CachedDevice.DeviceIP;
					return true;
				}
			}
		}

		return false;
	}

	TConstArrayView<FRivermaxDeviceInfo> FRivermaxDeviceFinder::GetDevices() const
	{
		return Devices;
	}

	void FRivermaxDeviceFinder::FindDevices()
	{
#if PLATFORM_WINDOWS
		constexpr ULONG Flags = GAA_FLAG_INCLUDE_PREFIX | GAA_FLAG_SKIP_MULTICAST | GAA_FLAG_SKIP_ANYCAST | GAA_FLAG_SKIP_DNS_SERVER | GAA_FLAG_SKIP_FRIENDLY_NAME;
		constexpr ULONG Family = AF_INET;

		// Based on SocketBSD implementation
		// determine the required size of the address list buffer
		ULONG Size = 0;
		ULONG Result = GetAdaptersAddresses(Family, Flags, NULL, NULL, &Size);

		if (Result != ERROR_BUFFER_OVERFLOW)
		{
			return;
		}

		PIP_ADAPTER_ADDRESSES AdapterAddresses = (PIP_ADAPTER_ADDRESSES)FMemory::Malloc(Size);

		// get the actual list of adapters
		Result = GetAdaptersAddresses(Family, Flags, NULL, AdapterAddresses, &Size);

		if (Result == ERROR_SUCCESS)
		{
			// extract the list of physical addresses from each adapter
			for (PIP_ADAPTER_ADDRESSES AdapterAddress = AdapterAddresses; AdapterAddress != NULL; AdapterAddress = AdapterAddress->Next)
			{
				if (AdapterAddress->OperStatus == IfOperStatusUp)
				{
					const auto AdapterFilterFunc = [](const FString& Description) -> bool
					{
						// We could also look for Mellanox but it would also include management port which we would need to discard
						if (Description.Contains(TEXT("ConnectX")))
						{
							return true;
						}

						return false;
					};

					const FString Description = StringCast<TCHAR>(AdapterAddress->Description).Get();
					if (AdapterFilterFunc(Description))
					{
						for (PIP_ADAPTER_UNICAST_ADDRESS UnicastAddress = AdapterAddress->FirstUnicastAddress; UnicastAddress != NULL; UnicastAddress = UnicastAddress->Next)
						{
							sockaddr_storage* RawAddress = reinterpret_cast<sockaddr_storage*>(UnicastAddress->Address.lpSockaddr);

							// Verify if it's IPV4 or 6
							if (RawAddress->ss_family == AF_INET)
							{
								const sockaddr* AdapterUniAddress = reinterpret_cast<const sockaddr*>(RawAddress);
								char IPString[NI_MAXHOST];
								if (getnameinfo(AdapterUniAddress, sizeof(sockaddr_in), IPString, NI_MAXHOST, nullptr, 0, NI_NUMERICHOST) == 0)
								{
									FRivermaxDeviceInfo NewDevice;
									NewDevice.Description = Description;
									NewDevice.InterfaceAddress = StringCast<TCHAR>(IPString).Get();

									UE_LOGF(LogRivermax, Log, "Found adapter: Name: '%ls', IP: '%ls'", *NewDevice.Description, *NewDevice.InterfaceAddress);
									Devices.Add(MoveTemp(NewDevice));
								}
							}
						}
					}
				}
			}
		}

		FMemory::Free(AdapterAddresses);

#elif PLATFORM_LINUX

		// On Linux, enumerate interfaces via getifaddrs and detect Mellanox ConnectX NICs by reading the PCI vendor ID from sysfs (/sys/class/net/<iface>/device/vendor).
		// Mellanox Technologies vendor ID is 0x15b3.
		struct ifaddrs* InterfaceList = nullptr;
		if (getifaddrs(&InterfaceList) != 0)
		{
			UE_LOG(LogRivermax, Warning, TEXT("getifaddrs() failed: no Rivermax devices found."));
			return;
		}

		for (struct ifaddrs* Interface = InterfaceList; Interface != nullptr; Interface = Interface->ifa_next)
		{
			// Skip interfaces that have no address or are not IPv4
			if (Interface->ifa_addr == nullptr || Interface->ifa_addr->sa_family != AF_INET)
			{
				continue;
			}

			// Skip interfaces that are not up or have no active link (e.g. unplugged cable)
			if (!(Interface->ifa_flags & IFF_UP) || !(Interface->ifa_flags & IFF_RUNNING))
			{
				continue;
			}

			const FString InterfaceName = UTF8_TO_TCHAR(Interface->ifa_name);

			// Read the PCI vendor ID from sysfs to identify Mellanox ConnectX adapters. Use POSIX open/read directly — FFileHelper::LoadFileToString can fail on sysfs
			// files because UE's file abstraction layer may not handle them correctly.
			const FString VendorPath = FString::Printf(TEXT("/sys/class/net/%s/device/vendor"), *InterfaceName);
			const int VendorFd = open(TCHAR_TO_UTF8(*VendorPath), O_RDONLY);
			if (VendorFd < 0)
			{
				// No sysfs device/vendor entry: not a PCI device (e.g. loopback, tunnel, virtual bridge).
				// Rivermax-capable ConnectX NICs are always PCI and will have this entry.
				continue;
			}
			char VendorBuf[32] = {};
			const ssize_t BytesRead = read(VendorFd, VendorBuf, sizeof(VendorBuf) - 1);
			close(VendorFd);
			if (BytesRead <= 0)
			{
				continue;
			}
			// Trim trailing newline
			VendorBuf[strcspn(VendorBuf, "\n\r")] = '\0';

			// Mellanox Technologies PCI vendor ID
			constexpr int32 MellanoxVendorID = 0x15b3;
			const int32 VendorID = static_cast<int32>(strtol(VendorBuf, nullptr, 0));
			if (VendorID != MellanoxVendorID)
			{
				continue;
			}

			// Convert binary address to dotted-decimal string
			const sockaddr_in* SockAddr = reinterpret_cast<const sockaddr_in*>(Interface->ifa_addr);
			char IPString[INET_ADDRSTRLEN];
			if (inet_ntop(AF_INET, &SockAddr->sin_addr, IPString, INET_ADDRSTRLEN) != nullptr)
			{
				FRivermaxDeviceInfo NewDevice;
				NewDevice.Description = FString::Printf(TEXT("ConnectX (%s)"), *InterfaceName);
				NewDevice.InterfaceAddress = UTF8_TO_TCHAR(IPString);

				UE_LOG(LogRivermax, Log, TEXT("Found adapter: Name: '%s', IP: '%s'"), *NewDevice.Description, *NewDevice.InterfaceAddress);
				Devices.Add(MoveTemp(NewDevice));
			}
		}

		freeifaddrs(InterfaceList);

#endif // PLATFORM_WINDOWS || PLATFORM_LINUX
	}


}

