// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Interfaces/IPv4/IPv4Address.h"
#include "IRivermaxManager.h"


namespace UE::RivermaxCore::Private
{
	class FRivermaxDeviceFinder
	{
	public:
		FRivermaxDeviceFinder();

		/** Tests whether provided string is a valid IP, containing 4 numeric values or wildcard (*) separated by dots. */
		bool IsValidIP(const FString& SourceIP) const;

		/** Resolves IP to a found Rivermax device interface. */
		bool ResolveIP(const FString& SourceIP, FString& OutDeviceIP) const;

		/** Returns list of found devices that can be used */
		TConstArrayView<FRivermaxDeviceInfo> GetDevices() const;

	private:

		/** Find all interfaces available */
		void FindDevices();

	private:

		struct FQueriedIPInfo
		{
			/** Whether or not this IP was found as valid */
			bool bIsValidIP = false;

			/** String representation of the IP */
			FString SourceIP;

			/** String representation of IP component separated by '.' */
			TArray<FString> Tokens;

			/** Hash of the matching found resolved device IP used to quickly find association */
			TOptional<uint32> ResolvedDeviceIPHash;
		};

		struct FDeviceInfo
		{
			/** String representation of the IP */
			FString DeviceIP;

			/** String representation of IP component separated by '.' */
			TArray<FString> Tokens;

			/** IPv4 representation of the IP with numerical tokens */
			FIPv4Address Address;

			/** Array representation of IP in numerical tokens to iterate over */
			uint8 AddressTokens[4];
		};

		/** Cache of IPs that were queried to see which were valid / resolved */
		mutable TMap<uint32, FQueriedIPInfo> QueriesCache;

		/** Cache of found Rivermax devices with useful representation of its IP */
		mutable TMap<uint32, FDeviceInfo> DevicesCache;
		
		/** List of found Rivermax devices */
		TArray<FRivermaxDeviceInfo> Devices;
	};
}


