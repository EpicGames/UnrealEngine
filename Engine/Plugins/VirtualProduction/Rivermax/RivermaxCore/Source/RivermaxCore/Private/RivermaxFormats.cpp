// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxFormats.h"

namespace UE::RivermaxCore
{
	const TArray<FVideoFormatInfo> FStandardVideoFormat::AllSamplingTypes =
	{
	//	FVideoFormatInfo { Sampling,						PixelDepth, PixelGroupSize, PixelGroupCoverage } }
		FVideoFormatInfo { ESamplingType::YUV422_8bit,		8,			4,				2 },
		FVideoFormatInfo { ESamplingType::YUV422_10bit,		10,			5,				2 },
		FVideoFormatInfo { ESamplingType::YUV422_12bit,		12,			6,				2 },
		FVideoFormatInfo { ESamplingType::YUV422_16bit,		16,			8,				2 },
		FVideoFormatInfo { ESamplingType::YUV422_16bitFloat,16,			8,				2 },
		FVideoFormatInfo { ESamplingType::YUV444_8bit,		8,			3,				1 },
		FVideoFormatInfo { ESamplingType::YUV444_10bit,		10,			15,				4 },
		FVideoFormatInfo { ESamplingType::YUV444_12bit,		12,			9,				2 },
		FVideoFormatInfo { ESamplingType::YUV444_16bit,		16,			6,				1 },
		FVideoFormatInfo { ESamplingType::YUV444_16bitFloat,16,			6,				1 },
		FVideoFormatInfo { ESamplingType::RGB_8bit,			8,			3,				1 },
		FVideoFormatInfo { ESamplingType::RGB_10bit,		10,			15,				4 },
		FVideoFormatInfo { ESamplingType::RGB_12bit,		12,			9,				2 },
		FVideoFormatInfo { ESamplingType::RGB_16bit,		16,			6,				1 },
		FVideoFormatInfo { ESamplingType::RGB_16bitFloat,	16,			6,				1 },
	};

	FVideoFormatInfo FStandardVideoFormat::GetVideoFormatInfo(ESamplingType SamplingType)
	{
		if (const FVideoFormatInfo* FoundInfo = AllSamplingTypes.FindByPredicate([SamplingType](const FVideoFormatInfo& Other) { return Other.Sampling == SamplingType; }))
		{
			return *FoundInfo;
		}

		return FVideoFormatInfo();
	}

	bool FStandardVideoFormat::IsRGB(ESamplingType SamplingType)
	{
		switch (SamplingType)
		{
		case ESamplingType::RGB_8bit:
		case ESamplingType::RGB_10bit:
		case ESamplingType::RGB_12bit:
		case ESamplingType::RGB_16bit:
		case ESamplingType::RGB_16bitFloat:
		{
			return true;
		}
		default:
		{
			return false;
		}
		};
	}

	bool FStandardVideoFormat::IsYUV(ESamplingType SamplingType)
	{
		switch (SamplingType)
		{
		case ESamplingType::YUV422_8bit:
		case ESamplingType::YUV422_10bit:
		case ESamplingType::YUV422_12bit:
		case ESamplingType::YUV422_16bit:
		case ESamplingType::YUV422_16bitFloat:
		case ESamplingType::YUV444_8bit:
		case ESamplingType::YUV444_10bit:
		case ESamplingType::YUV444_12bit:
		case ESamplingType::YUV444_16bit:
		case ESamplingType::YUV444_16bitFloat:
		{
			return true;
		}
		default:
		{
			return false;
		}
		};
	}

	bool FStandardVideoFormat::IsYUV422(ESamplingType SamplingType)
	{
		switch (SamplingType)
		{
		case ESamplingType::YUV422_8bit:
		case ESamplingType::YUV422_10bit:
		case ESamplingType::YUV422_12bit:
		case ESamplingType::YUV422_16bit:
		case ESamplingType::YUV422_16bitFloat:
		{
			return true;
		}
		default:
		{
			return false;
		}
		};
	}

	bool FStandardVideoFormat::IsYUV444(ESamplingType SamplingType)
	{
		switch (SamplingType)
		{
		case ESamplingType::YUV444_8bit:
		case ESamplingType::YUV444_10bit:
		case ESamplingType::YUV444_12bit:
		case ESamplingType::YUV444_16bit:
		case ESamplingType::YUV444_16bitFloat:
		{
			return true;
		}
		default:
		{
			return false;
		}
		};
	}
}

	
