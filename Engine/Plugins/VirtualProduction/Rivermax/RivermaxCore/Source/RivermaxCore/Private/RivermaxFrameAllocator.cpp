// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxFrameAllocator.h"

#include "Async/ParallelFor.h"
#include "RivermaxLog.h"
#include "IRivermaxOutputStream.h"

#if PLATFORM_WINDOWS
#include "CudaModule.h"
#include "ID3D12DynamicRHI.h"
#endif // PLATFORM_WINDOWS

namespace UE::RivermaxCore::Private
{
	static TAutoConsoleVariable<int32> CVarRivermaxOutputEnableParallelCopy(
		TEXT("Rivermax.Output.EnableParallelCopy"), 1,
		TEXT("Parallel"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputParallelCopyThreadCount(
		TEXT("Rivermax.Output.ParallelCopyThreadCount"), 4,
		TEXT("Parallel"),
		ECVF_Default);

#if PLATFORM_WINDOWS

	FGPUAllocator::FGPUAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate)
		: Super(InDesiredFrameSize, InOnDataCopiedDelegate)
	{

	}

	bool FGPUAllocator::Allocate(int32 FrameCount, bool bAlignFrameMemory)
	{
		// Allocate a single memory space that will contain all frame buffers
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::GPUAllocation);

		FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
		CudaModule.DriverAPI()->cuCtxPushCurrent(CudaModule.GetCudaContext());

		// Todo: Add support for mgpu. For now, this will not work unless the memcpy does implicitely a cross gpu transfer.
		const int GPUIndex = CudaModule.GetCudaDeviceIndex();
		CUdevice CudaDevice;
		CUresult Status = CudaModule.DriverAPI()->cuDeviceGet(&CudaDevice, GPUIndex);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to get a Cuda device for GPU %d. Status: %d", GPUIndex, Status);
			return false;
		}

		CUmemAllocationProp AllocationProperties = {};
		AllocationProperties.type = CU_MEM_ALLOCATION_TYPE_PINNED;
		AllocationProperties.allocFlags.gpuDirectRDMACapable = 1;
		AllocationProperties.allocFlags.usage = 0;
		AllocationProperties.location.type = CU_MEM_LOCATION_TYPE_DEVICE;
		AllocationProperties.location.id = CudaDevice;

		// Get memory granularity required for cuda device. We need to align allocation with this.
		size_t Granularity;
		Status = CudaModule.DriverAPI()->cuMemGetAllocationGranularity(&Granularity, &AllocationProperties, CU_MEM_ALLOC_GRANULARITY_RECOMMENDED);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to get allocation granularity. Status: %d", Status);
			return false;
		}

		// Cuda requires allocated memory to be aligned with a certain granularity
		// We align each frame size to the desired granularity and multiply that by number of buffer
		// This causes more memory to be allocated but doing a single allocation fails rmax stream creation
		const size_t CudaAlignedFrameSize = (DesiredFrameSize % Granularity) ? DesiredFrameSize + (Granularity - (DesiredFrameSize % Granularity)) : DesiredFrameSize;
		const size_t TotalCudaAllocSize = CudaAlignedFrameSize * FrameCount;

		// Reserve contiguous memory to contain required number of buffers. 
		CUdeviceptr CudaBaseAddress;
		constexpr CUdeviceptr InitialAddress = 0;
		constexpr int32 Flags = 0;
		Status = CudaModule.DriverAPI()->cuMemAddressReserve(&CudaBaseAddress, TotalCudaAllocSize, Granularity, InitialAddress, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to reserve memory for %zu bytes. Status: %d", TotalCudaAllocSize, Status);
			return false;
		}

		// Make the allocation on device memory
		CUmemGenericAllocationHandle Handle;
		Status = CudaModule.DriverAPI()->cuMemCreate(&Handle, TotalCudaAllocSize, &AllocationProperties, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to create memory on device. Status: %d", Status);
			return false;
		}

		bool bExit = false;
		constexpr int32 Offset = 0;
		Status = CudaModule.DriverAPI()->cuMemMap(CudaBaseAddress, TotalCudaAllocSize, Offset, Handle, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to map memory. Status: %d", Status);
			// Need to release handle no matter what
			bExit = true;
		}

		// Cache to know we need to unmap/deallocate even if it fails down the road
		CudaAllocatedMemory = TotalCudaAllocSize;
		CudaAllocatedMemoryBaseAddress = reinterpret_cast<void*>(CudaBaseAddress);

		Status = CudaModule.DriverAPI()->cuMemRelease(Handle);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't allocate GPUMemory. Failed to release handle. Status: %d", Status);
			return false;
		}

		if (bExit)
		{
			return false;
		}

		// Setup access description.
		CUmemAccessDesc MemoryAccessDescription = {};
		MemoryAccessDescription.flags = CU_MEM_ACCESS_FLAGS_PROT_READWRITE;
		MemoryAccessDescription.location.type = CU_MEM_LOCATION_TYPE_DEVICE;
		MemoryAccessDescription.location.id = CudaDevice;
		constexpr int32 Count = 1;
		Status = CudaModule.DriverAPI()->cuMemSetAccess(CudaBaseAddress, TotalCudaAllocSize, &MemoryAccessDescription, Count);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize output to use GPUDirect. Failed to configure memory access. Status: %d", Status);
			return false;
		}

		CUstream CudaStream;
		Status = CudaModule.DriverAPI()->cuStreamCreate(&CudaStream, CU_STREAM_NON_BLOCKING);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize output to use GPUDirect. Failed to create its stream. Status: %d", Status);
			return false;
		}

		GPUStream = CudaStream;

		Status = CudaModule.DriverAPI()->cuCtxSynchronize();
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize output to use GPUDirect. Failed to synchronize context. Status: %d", Status);
			return false;
		}

		AllocatedFrameCount = FrameCount;

		if (bAlignFrameMemory)
		{
			AllocatedFrameSize = CudaAlignedFrameSize;
		}
		else
		{
			AllocatedFrameSize = DesiredFrameSize;
		}

		CudaModule.DriverAPI()->cuCtxPopCurrent(nullptr);

		return true;
	}

	void FGPUAllocator::Deallocate()
	{
		if (CudaAllocatedMemory > 0)
		{
			FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
			CudaModule.DriverAPI()->cuCtxPushCurrent(CudaModule.GetCudaContext());

			for (const TPair<FBufferRHIRef, void*>& Entry : BufferCudaMemoryMap)
			{
				if (Entry.Value)
				{
					CudaModule.DriverAPI()->cuMemFree(reinterpret_cast<CUdeviceptr>(Entry.Value));
				}
			}
			BufferCudaMemoryMap.Empty();

			const CUdeviceptr BaseAddress = reinterpret_cast<CUdeviceptr>(CudaAllocatedMemoryBaseAddress);
			CUresult Status = CudaModule.DriverAPI()->cuMemUnmap(BaseAddress, CudaAllocatedMemory);
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to unmap cuda memory. Status: %d", Status);
			}

			Status = CudaModule.DriverAPI()->cuMemAddressFree(BaseAddress, CudaAllocatedMemory);
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to free cuda memory. Status: %d", Status);
			}

			Status = CudaModule.DriverAPI()->cuStreamDestroy(reinterpret_cast<CUstream>(GPUStream));
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to destroy cuda stream. Status: %d", Status);
			}
			GPUStream = nullptr;
			AllocatedFrameCount = 0;

			CudaModule.DriverAPI()->cuCtxPopCurrent(nullptr);
		}
	}

	void* FGPUAllocator::GetFrameAddress(int32 FrameIndex) const
	{
		if (FrameIndex >= 0 && FrameIndex < AllocatedFrameCount)
		{
			return reinterpret_cast<uint8*>(CudaAllocatedMemoryBaseAddress) + (FrameIndex * AllocatedFrameSize);
		}

		return nullptr;
	}

	bool FGPUAllocator::CopyData(const FCopyArgs& Args)
	{
		check(Args.RHISourceMemory || Args.SourceMemory);

		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::CudaCopyStart);
		{
			FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
			CUresult Result = CudaModule.DriverAPI()->cuCtxPushCurrent(CudaModule.GetCudaContext());

			ON_SCOPE_EXIT
			{
				FCUDAModule::CUDA().cuCtxPopCurrent(nullptr); 
			};

			// If source memory is rhi memory, we need to go through mapping, otherwise, it's already cuda located
			void* SourceMemory = Args.SourceMemory;
			if (SourceMemory == nullptr)
			{
				SourceMemory = GetMappedAddress(Args.RHISourceMemory);
				if (SourceMemory == nullptr)
				{
					UE_LOGF(LogRivermax, Error, "Failed to find a mapped memory address for captured buffer. Stopping capture.");
					return false;
				}
			}

			const CUdeviceptr DestinationCudaPointer = reinterpret_cast<CUdeviceptr>(Args.DestinationMemory);
			const CUdeviceptr BaseCudaPointer = reinterpret_cast<CUdeviceptr>(CudaAllocatedMemoryBaseAddress);
			if(DestinationCudaPointer < BaseCudaPointer || DestinationCudaPointer >= (BaseCudaPointer + CudaAllocatedMemory))
			{
				UE_LOGF(LogRivermax, Error, "Trying to copy data outside of allocated buffer.");
				return false;
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::CudaMemcopyAsync);
				const CUdeviceptr CudaMemoryPointer = reinterpret_cast<CUdeviceptr>(SourceMemory);
				Result = CudaModule.DriverAPI()->cuMemcpyDtoDAsync(DestinationCudaPointer, CudaMemoryPointer, Args.SizeToCopy, reinterpret_cast<CUstream>(GPUStream));
				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogRivermax, Error, "Failed to copy captured buffer to cuda memory. Stopping capture. Error: %d", Result);
					return false;
				}
			}
			

			// Callback called by Cuda when stream work has completed on cuda engine (MemCpy -> Callback)
			// Once Memcpy has been done, we know we can mark that memory as available to be sent. 

			auto CudaCallback = [](void* UserData)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::CudaCopyDone);

				FGPUAllocator* Allocator = reinterpret_cast<FGPUAllocator*>(UserData);
				TOptional<TSharedPtr<FBaseDataCopySideCar>> Sidecar = Allocator->PendingCopies.Dequeue();
				if (Sidecar.IsSet())
				{
					Allocator->OnDataCopiedDelegate.ExecuteIfBound(Sidecar.GetValue());
				}
			};

			// Add pending payload for cuda callback 
			PendingCopies.Enqueue(Args.SideCar);

			// Schedule a callback to make the frame available
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::CudaLaunchHost);
				CudaModule.DriverAPI()->cuLaunchHostFunc(reinterpret_cast<CUstream>(GPUStream), CudaCallback, this);
			}

			return true;
		}
	}

	void* FGPUAllocator::GetMappedAddress(const FBufferRHIRef& InBuffer)
	{
		// If we are here, d3d12 had to have been validated
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();
		check(RHIType == ERHIInterfaceType::D3D12);

		//Do we already have a mapped address for this buffer
		if (BufferCudaMemoryMap.Find((InBuffer)) == nullptr)
		{
			int64 BufferMemorySize = 0;
			CUexternalMemory MappedExternalMemory = nullptr;
			HANDLE D3D12BufferHandle = 0;
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC CudaExtMemHandleDesc = {};

			// Create shared handle for our buffer
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax_D3D12CreateSharedHandle);

				ID3D12Resource* NativeD3D12Resource = GetID3D12DynamicRHI()->RHIGetResource(InBuffer);
				BufferMemorySize = GetID3D12DynamicRHI()->RHIGetResourceMemorySize(InBuffer);

				TRefCountPtr<ID3D12Device> OwnerDevice;
				HRESULT QueryResult;
				if ((QueryResult = NativeD3D12Resource->GetDevice(IID_PPV_ARGS(OwnerDevice.GetInitReference()))) != S_OK)
				{
					UE_LOGF(LogRivermax, Error, "Failed to get D3D12 device for captured buffer ressource: %d)", QueryResult);
					return nullptr;
				}

				if ((QueryResult = OwnerDevice->CreateSharedHandle(NativeD3D12Resource, NULL, GENERIC_ALL, NULL, &D3D12BufferHandle)) != S_OK)
				{
					UE_LOGF(LogRivermax, Error, "Failed to create shared handle for captured buffer ressource: %d", QueryResult);
					return nullptr;
				}

				CudaExtMemHandleDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D12_RESOURCE;
				CudaExtMemHandleDesc.handle.win32.name = nullptr;
				CudaExtMemHandleDesc.handle.win32.handle = D3D12BufferHandle;
				CudaExtMemHandleDesc.size = BufferMemorySize;
				CudaExtMemHandleDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax_CudaImportMemory);

				const CUresult Result = FCUDAModule::CUDA().cuImportExternalMemory(&MappedExternalMemory, &CudaExtMemHandleDesc);

				if (D3D12BufferHandle)
				{
					CloseHandle(D3D12BufferHandle);
				}

				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogRivermax, Error, "Failed to import shared buffer. Error: %d", Result);
					return nullptr;
				}
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax_MapCudaMemory);

				CUDA_EXTERNAL_MEMORY_BUFFER_DESC BufferDescription = {};
				BufferDescription.offset = 0;
				BufferDescription.size = BufferMemorySize;
				CUdeviceptr NewMemory;
				const CUresult Result = FCUDAModule::CUDA().cuExternalMemoryGetMappedBuffer(&NewMemory, MappedExternalMemory, &BufferDescription);
				if (Result != CUDA_SUCCESS || NewMemory == 0)
				{
					UE_LOGF(LogRivermax, Error, "Failed to get shared buffer mapped memory. Error: %d", Result);
					return nullptr;
				}

				BufferCudaMemoryMap.Add(InBuffer, reinterpret_cast<void*>(NewMemory));
			}
		}

		// At this point, we have the mapped buffer in cuda space and we can use it to schedule a memcpy on cuda engine.
		return BufferCudaMemoryMap[InBuffer];
	}

#endif // PLATFORM_WINDOWS

	bool FSystemAllocator::Allocate(int32 FrameCount, bool bAlignFrameMemory)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::SystemAllocation);

		constexpr uint32 CacheLineSize = PLATFORM_CACHE_LINE_SIZE;

		AllocatedFrameSize = bAlignFrameMemory ? Align(DesiredFrameSize, CacheLineSize) : DesiredFrameSize;
		SystemMemoryBaseAddress = FMemory::Malloc(AllocatedFrameSize * FrameCount, CacheLineSize);

		if (SystemMemoryBaseAddress)
		{
			AllocatedFrameCount = FrameCount;
			return true;
		}

		return false;
	}

	void FSystemAllocator::Deallocate()
	{
		if (SystemMemoryBaseAddress)
		{
			FMemory::Free(SystemMemoryBaseAddress);
			SystemMemoryBaseAddress = nullptr;
			AllocatedFrameCount = 0;
		}
	}

	void* FSystemAllocator::GetFrameAddress(int32 FrameIndex) const
	{
		if (FrameIndex >= 0 && FrameIndex < AllocatedFrameCount)
		{
			return reinterpret_cast<uint8*>(SystemMemoryBaseAddress) + (FrameIndex * AllocatedFrameSize);
		}

		return nullptr;
	}

	FSystemAllocator::FSystemAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate)
		: Super(InDesiredFrameSize, InOnDataCopiedDelegate)
	{
		bUseParallelFor = CVarRivermaxOutputEnableParallelCopy.GetValueOnAnyThread() != 0;
		ParallelForThreadCount = FMath::Clamp(CVarRivermaxOutputParallelCopyThreadCount.GetValueOnAnyThread(), 1, 100);
	}

	bool FSystemAllocator::CopyData(const FCopyArgs& Args)
	{
		// GPU memory copy shouldn't happen for system allocator
		if (Args.RHISourceMemory != nullptr)
		{
			ensure(false);
			return false;
		}


		if(bUseParallelFor)
		{
			const int32 ThreadCount = ParallelForThreadCount;
			const uint32 SizePerThread = 1 + ((Args.SizeToCopy - 1) / ThreadCount);
			ParallelFor(ThreadCount, [SizePerThread, &Args](int32 BlockIndex)
			{
				const uint32 Offset = BlockIndex * SizePerThread;
				const uint32 MaxSize = Args.SizeToCopy - Offset;
				const uint32 CopySize = FMath::Min(SizePerThread, MaxSize);
				uint8* SourceAddress = reinterpret_cast<uint8*>(Args.SourceMemory) + Offset;
				uint8* DestinationAddress = reinterpret_cast<uint8*>(Args.DestinationMemory) + Offset;
				FMemory::Memcpy(DestinationAddress, SourceAddress, CopySize);
			});
		}
		else
		{
			FMemory::Memcpy(Args.DestinationMemory, Args.SourceMemory, Args.SizeToCopy);
		}
		
		OnDataCopiedDelegate.ExecuteIfBound(Args.SideCar);

		return true;
	}

	FBaseFrameAllocator::FBaseFrameAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate)
		: DesiredFrameSize(InDesiredFrameSize)
		, OnDataCopiedDelegate(MoveTemp(InOnDataCopiedDelegate))
	{

	}
}

