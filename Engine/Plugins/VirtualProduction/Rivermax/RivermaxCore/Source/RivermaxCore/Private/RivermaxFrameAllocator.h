// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/SpscQueue.h"
#include "Delegates/DelegateCombinations.h"
#include "RivermaxOutputFrame.h"
#include "RHI.h"

namespace UE::RivermaxCore
{
	struct FRivermaxOutputInfoVideo;
}


namespace UE::RivermaxCore::Private
{
	/** Sidecar base struct given back to callback after data has been copied */
	struct FBaseDataCopySideCar
	{
		virtual ~FBaseDataCopySideCar() = default;
	};

	/** Arguments describing memcopy operation required from allocator */
	struct FCopyArgs
	{
		/** RHI source memory that will be mapped to cuda */
		FBufferRHIRef RHISourceMemory;
		
		/** Source memory that doesn't need to be mapped since it's in the same space as destination*/
		void* SourceMemory = nullptr;
		
		/** Destination memory where data has to be copied */
		void* DestinationMemory = nullptr;
		
		/** Bytes needed to be copied */
		SIZE_T SizeToCopy = 0;
		
		/** Sidecar to be re-provided on callback after memory has been copied */
		TSharedPtr<FBaseDataCopySideCar> SideCar;
	};

	class FRivermaxOutputFrame;

	/** Delegate triggered when data has been copied into a given frame */
	DECLARE_DELEGATE_OneParam(FOnFrameDataCopiedDelegate, const TSharedPtr<FBaseDataCopySideCar>&);

	/** Base class for frame allocation. Currently supports GPU allocation using CUDA and System memory */
	class FBaseFrameAllocator
	{
	public:
		FBaseFrameAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate);
		virtual ~FBaseFrameAllocator() = default;

		/** Allocates memory to hold FrameCount frames. 
		 *  If bAlignFrameMemory is true, aligns each frame memory size to allocator alignment
		 *  If not, aligns entire memory block instead
		 */
		virtual bool Allocate(int32 FrameCount, bool bAlignFrameMemory) = 0;

		/** Deallocate memory */
		virtual void Deallocate() = 0;

		/** Initiate a copy into a given frame using Args description of the operation */
		virtual bool CopyData(const FCopyArgs& Args) = 0;
		
		/** Returns buffer address of a given frame */
		virtual void* GetFrameAddress(int32 FrameIndex) const = 0;

	protected:

		/** Total desired size of a frame */
		int32 DesiredFrameSize = 0;

		/** Delegate callbacked when data has been copied. Depending on allocator, memcopy can be async. */
		FOnFrameDataCopiedDelegate OnDataCopiedDelegate;
	};

#if PLATFORM_WINDOWS
	class FGPUAllocator : public FBaseFrameAllocator
	{
		using Super = FBaseFrameAllocator;

	public:
		FGPUAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate);

		//~ Begin FBaseFrameAllocator interface
		virtual bool Allocate(int32 FrameCount, bool bAlignFrameMemory) override;
		virtual void Deallocate() override;
		virtual bool CopyData(const FCopyArgs& Args) override;
		virtual void* GetFrameAddress(int32 FrameIndex) const override;
	
	protected:
		//~ End FBaseFrameAllocator interface

	private:
		/** Get mapped address in cuda space for a given buffer. Cache will be updated if not found */
		void* GetMappedAddress(const FBufferRHIRef& InBuffer);

	private:

		/** Number of frames that was allocated */
		int32 AllocatedFrameCount = 0;

		/** Allocated size of a frame */
		size_t AllocatedFrameSize = 0;

		/** Allocated memory base address used when it's time to free */
		void* CudaAllocatedMemoryBaseAddress = nullptr;

		/** Total allocated gpu memory. */
		int32 CudaAllocatedMemory = 0;

		/** Cuda stream used for our operations */
		void* GPUStream = nullptr;

		/** Map between buffer we are sending and their mapped address in gpu space */
		TMap<FBufferRHIRef, void*> BufferCudaMemoryMap;

		/** Sidecar for data copied callback that are waiting for a cuda memcopy. Will be dequeued when our cuda task has completed */
		TSpscQueue<TSharedPtr<FBaseDataCopySideCar>> PendingCopies;
	};
#endif // PLATFORM_WINDOWS

	class FSystemAllocator : public FBaseFrameAllocator
	{
		using Super = FBaseFrameAllocator;

	public:
		FSystemAllocator(int32 InDesiredFrameSize, FOnFrameDataCopiedDelegate InOnDataCopiedDelegate);
		
		//~ Begin FBaseFrameAllocator interface
		virtual bool Allocate(int32 FrameCount, bool bAlignFrameMemory) override;
		virtual void Deallocate() override;
		virtual bool CopyData(const FCopyArgs& Args) override;
		virtual void* GetFrameAddress(int32 FrameIndex) const override;
		//~ End FBaseFrameAllocator interface
	
	private:

		/** Number of frames that was allocated */
		int32 AllocatedFrameCount = 0;

		/** Size of a frame that was allocated */
		size_t AllocatedFrameSize = 0;

		/** Total memory allocated to hold all frames */
		size_t SystemAllocatedMemory = 0;

		/** Allocated memory base address used when it's time to free */
		void* SystemMemoryBaseAddress = nullptr;

		/** Cached value for cvar controlling whether parallelfor is used to memcopy */
		bool bUseParallelFor = false;

		/** Cached value for cvar controlling number of threads to use when doing parallelfor memcopy */
		int32 ParallelForThreadCount = 4;
	};
}


