// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxFrameManager.h"

#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RivermaxLog.h"
#include "RivermaxTracingUtils.h"

namespace UE::RivermaxCore::Private
{
	/** Sidecar used when initiating memcopy. We provide the frame involved to update its state. */
	struct FFrameBufferCopyInfo : public FBaseDataCopySideCar
	{
		TSharedPtr<FRivermaxOutputFrame> CopiedFrame;
	};

	FFrameManager::~FFrameManager()
	{
		Cleanup();
	}

	EFrameMemoryLocation FFrameManager::Initialize(const FFrameManagerSetupArgs& Args)
	{
		RivermaxManager = FModuleManager::GetModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore")).GetRivermaxManager();
		check(RivermaxManager);

		OnFrameReadyDelegate = Args.OnFrameReadyDelegate;
		OnPreFrameReadyDelegate = Args.OnPreFrameReadyDelegate;
		OnFreeFrameDelegate = Args.OnFreeFrameDelegate;
		OnCriticalErrorDelegate = Args.OnCriticalErrorDelegate;
		FrameResolution = Args.Resolution;
		TotalFrameCount = Args.NumberOfFrames;
		FOnFrameDataCopiedDelegate OnDataCopiedDelegate = FOnFrameDataCopiedDelegate::CreateRaw(this, &FFrameManager::OnDataCopied);

#if PLATFORM_WINDOWS
		if (Args.bTryGPUAllocation)
		{
			FrameAllocator = MakeUnique<FGPUAllocator>(Args.FrameDesiredSize, OnDataCopiedDelegate);
			if (FrameAllocator->Allocate(TotalFrameCount, Args.bAlignEachFrameAlloc))
			{
				MemoryLocation = EFrameMemoryLocation::GPU;
			}
		}
#endif // PLATFORM_WINDOWS

		if (MemoryLocation == EFrameMemoryLocation::None)
		{
			FrameAllocator = MakeUnique<FSystemAllocator>(Args.FrameDesiredSize, OnDataCopiedDelegate);
			if (FrameAllocator->Allocate(TotalFrameCount, Args.bAlignEachFrameAlloc))
			{
				MemoryLocation = EFrameMemoryLocation::System;
			}
		}

		if (MemoryLocation != EFrameMemoryLocation::None)
		{
			// Create frame pool.
			FramePool = MakeUnique<FRivermaxOutputFramePool>(TotalFrameCount);
			TArray<TSharedPtr<FRivermaxOutputFrame>> AllocatedFrames;
			for (uint32 Index = 0; Index < TotalFrameCount; ++Index)
			{
				/** Create actual frames and assign their video memory address from allocator */
				TSharedPtr<FRivermaxOutputFrame> Frame = FramePool->AcquireShared(true /*allocate memory*/);
				Frame->Buffer = FrameAllocator->GetFrameAddress(Index);
				AllocatedFrames.Add(Frame);
			}
			AllocatedFrames.Empty();
			FramePool->Tick();
		}

		return MemoryLocation;
	}

	void FFrameManager::Cleanup()
	{
		if (FrameAllocator)
		{
			FrameAllocator->Deallocate();
			FrameAllocator.Reset();
		}

		if (FramePool.IsValid())
		{
			FramePool->Reset();
			FramePool.Reset();
		}
	}

	TSharedPtr<FRivermaxOutputFrame> FFrameManager::GetFreeFrame()
	{
		return FramePool->AcquireShared(false /*don't allocate new items*/);
	}

	TSharedPtr<FRivermaxOutputFrame> FFrameManager::DequeueFrameToSend()
	{
		FScopeLock Lock(&ContainersCritSec);
		if (FramesToBeSent.IsEmpty())
		{
			return nullptr;
		}
		TSharedPtr<FRivermaxOutputFrame> ReadyToSendFrame;
		FramesToBeSent.Dequeue(ReadyToSendFrame);
		return MoveTemp(ReadyToSendFrame);
	}

	bool FFrameManager::IsFrameAvailableToSend()
	{
		FScopeLock Lock(&ContainersCritSec);
		return !FramesToBeSent.IsEmpty();
	}

	void FFrameManager::FrameSentEvent()
	{
		// Todo: the frame iself should call this when it is returned back to the pool.
		OnFreeFrameDelegate.ExecuteIfBound();
	}

	void FFrameManager::EnqueFrameToSend(const TSharedPtr<FRivermaxOutputFrame>& Frame)
	{
		{
			// Make frame available to be sent
			FScopeLock Lock(&ContainersCritSec);
			FramesToBeSent.Enqueue(Frame);
		}

		OnFrameReadyDelegate.ExecuteIfBound();
	}

	bool FFrameManager::SetFrameData(TSharedPtr<FRivermaxOutputInfoVideo> NewFrameInfo, TSharedPtr<FRivermaxOutputFrame> ReservedFrame)
	{
		bool bSuccess = false;
		check(ReservedFrame.IsValid());

		TSharedPtr<FFrameBufferCopyInfo> Sidecar = MakeShared<FFrameBufferCopyInfo>();
		Sidecar->CopiedFrame = ReservedFrame;

		FCopyArgs Args;
		Args.RHISourceMemory = NewFrameInfo->GPUBuffer;
		Args.SourceMemory = NewFrameInfo->CPUBuffer;
		Args.DestinationMemory = ReservedFrame->Buffer;
		Args.SizeToCopy = NewFrameInfo->Height * NewFrameInfo->Stride;
		Args.SideCar = MoveTemp(Sidecar);
		bSuccess = FrameAllocator->CopyData(Args);

		if (!bSuccess)
		{
			OnCriticalErrorDelegate.ExecuteIfBound();
		}
			 
		return bSuccess;
	}

	void FFrameManager::OnDataCopied(const TSharedPtr<FBaseDataCopySideCar>& Payload)
	{
		TSharedPtr<FFrameBufferCopyInfo> CopyInfo = StaticCastSharedPtr<FFrameBufferCopyInfo>(Payload);
		if (ensure(CopyInfo && CopyInfo->CopiedFrame))
		{
			//TODO: make sure that tracing is not reliant on frame index.
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutFrameReadyTraceEvents[CopyInfo->CopiedFrame->GetFrameCounter() % 10]);
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[CopyInfo->CopiedFrame->GetFrameCounter() % 10]);
			OnPreFrameReadyDelegate.ExecuteIfBound();
			EnqueFrameToSend(CopyInfo->CopiedFrame);
		}
	}
}

