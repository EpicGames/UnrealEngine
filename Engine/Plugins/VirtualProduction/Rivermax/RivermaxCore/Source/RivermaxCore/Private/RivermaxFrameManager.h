// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/SpscQueue.h"
#include "RivermaxFormats.h"
#include "RivermaxFrameAllocator.h"
#include "RivermaxOutputFrame.h"
#include "RivermaxTypes.h"
#include "MediaObjectPool.h"

namespace UE::RivermaxCore
{
	class IRivermaxManager;
}

namespace UE::RivermaxCore::Private
{
	class FBaseFrameAllocator;
	struct FBaseDataCopySideCar;

	/** Where frame memory is allocated */
	enum class EFrameMemoryLocation : uint8
	{
		/** No memory was allocated */
		None,

		/** Memory allocated in system memory */
		System,

		/** Memory allocated on GPU. Cuda space is used at the moment. */
		GPU
	};

	DECLARE_DELEGATE(FOnFrameReadyDelegate);
	DECLARE_DELEGATE(FOnPreFrameReadyDelegate);
	DECLARE_DELEGATE(FOnFreeFrameDelegate);
	DECLARE_DELEGATE(FOnCriticalErrorDelegate);

	/** Holds arguments to configure frame manager during initialization */
	struct FFrameManagerSetupArgs
	{
		/** Resolution of video frames to allocate */
		FIntPoint Resolution = FIntPoint::ZeroValue;

		/** Stride of a line of video frame */
		uint32 Stride = 0;

		/** Desired size for a frame. Can be greater than what is needed to align with Rivermax's chunks */
		uint32 FrameDesiredSize = 0;

		/** Whether allocator will align each frame to desired alignment or the entire block */
		bool bAlignEachFrameAlloc = false;

		/** Number of video frames required */
		uint8 NumberOfFrames = 0;

		/** Whether we should try allocating on GPU */
		bool bTryGPUAllocation = true;

		/** Delegate called when a frame is now free to use */
		FOnFreeFrameDelegate OnFreeFrameDelegate;

		/** Delegate triggered just before a frame is enqueued to be sent */
		FOnPreFrameReadyDelegate OnPreFrameReadyDelegate;
		
		/** Delegate called when a frame is now ready to be sent */
		FOnFrameReadyDelegate OnFrameReadyDelegate;

		/** Delegate called when a critical has happened and stream should shut down */
		FOnCriticalErrorDelegate OnCriticalErrorDelegate;
	};

	/**
	* Media object pool responsible for managing shared pointers. Pointers allocated by this class 
	* are returned to the pending pool where it is checked via IsReadyToBeUsed if it is ready to be used again.
	* It reduces manual thread/frame management required.
	*/
	class FRivermaxOutputFramePool : public TMediaObjectPool<FRivermaxOutputFrame> { };

	/** 
	 * Class managing frames that we output over network 
	 * Handles memory allocation and state tracking
	 * 
	 * States of a frame
	 * 
	 * Free :	Frame can be used by the capture system.
	 * Pending:	Frame is being used by the capture system. 
	 *			Data isn't ready to be sent out yet but it's reserved for a given identifier.
	 * Ready:	Frame is ready to be sent. Data has been copied into it. 
	 * Sending: Frame is being actively sent out the wire. Can't modify it until next frame boundary
	 *
	 * Frame rate control
	 * 
	 * Sending a frame out takes a full frame interval so if capture system goes faster than output rate
	 * Free frames list will get depleted. If frame locking mode is used, getting the next free frame
	 * will block until a new one is available which will happen at next frame boundary
	 * Rendering and capturing the next frame might be quick but when ready to present it, it will get stalled.
	 * This will cause the engine's frame rate to match the output frame rate.
	 * 
	 */
	class FFrameManager
	{
	public:
		virtual ~FFrameManager();

		/** Initializes frame manager with a set of options. Returns where frames were allocated. */
		EFrameMemoryLocation Initialize(const FFrameManagerSetupArgs& Args);
		
		/** Requests cleanup of allocated memory */
		void Cleanup();

		/** Returns a frame not being used */
		TSharedPtr<FRivermaxOutputFrame> GetFreeFrame();

		/** Returns next frame ready to be sent. Can be null. */
		TSharedPtr<FRivermaxOutputFrame> DequeueFrameToSend();
		
		/** Mark a frame as being ready to be sent */
		void EnqueFrameToSend(const TSharedPtr<FRivermaxOutputFrame>& Frame);

		/** Returns next frame ready to be sent. Can be null. */
		bool IsFrameAvailableToSend();

		/** Mark a frame as sent */
		void FrameSentEvent();

		/** Initiates memory copy for a given frame */
		bool SetFrameData(TSharedPtr<FRivermaxOutputInfoVideo> NewFrame, TSharedPtr<FRivermaxOutputFrame> ReservedFrame);

	private:

		/** Called back when copy request was completed by allocator */
		void OnDataCopied(const TSharedPtr<FBaseDataCopySideCar>& Sidecar);

	private:

		/** Resolution of video frames */
		FIntPoint FrameResolution = FIntPoint::ZeroValue;

		/** Number of frames allocated */
		uint32 TotalFrameCount = 0;

		/** Location of memory allocated */
		EFrameMemoryLocation MemoryLocation = EFrameMemoryLocation::None;

		/** Critical section to protect various frame containers */
		FCriticalSection ContainersCritSec;

		/** Frame allocator dealing with memory operation */
		TUniquePtr<FBaseFrameAllocator> FrameAllocator;

		/** 
		* Pool of allocated frames. Thread safe allocation. All items allocated via AcquireShared are returned back to the pool automatically.
		*/
		TUniquePtr<FRivermaxOutputFramePool> FramePool;

		/** Queue of frames that are ready to be sent. First in, first out. */
		TQueue<TSharedPtr<FRivermaxOutputFrame>> FramesToBeSent;

		/** Delegate triggered when a frame is free to use */
		FOnFreeFrameDelegate OnFreeFrameDelegate;

		/** Delegate triggered just before a frame is enqueued to be sent */
		FOnPreFrameReadyDelegate OnPreFrameReadyDelegate;
		
		/** Delegate triggered when a frame is ready to be sent (video data has been copied) */
		FOnFrameReadyDelegate OnFrameReadyDelegate;

		/** Delegate triggered when a critical error has happened and stream should shut down */
		FOnCriticalErrorDelegate OnCriticalErrorDelegate;

		/** Quick access to rivermax manager */
		TSharedPtr<IRivermaxManager> RivermaxManager;
	};
}


