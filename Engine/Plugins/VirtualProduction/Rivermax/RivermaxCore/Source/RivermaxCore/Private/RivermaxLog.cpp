// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxLog.h"

DEFINE_LOG_CATEGORY(LogRivermax);
