// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxManager.h"

#include "HAL/IConsoleManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IRivermaxCoreModule.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"
#include "RivermaxDeviceFinder.h"
#include "RivermaxLog.h"
#include "RivermaxTracingUtils.h"

#include <chrono>

#if PLATFORM_WINDOWS
#include "CudaModule.h"

#include "Windows/AllowWindowsPlatformTypes.h"

#include <ws2tcpip.h>

#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_LINUX
#include <arpa/inet.h>
#include <unistd.h>
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "RivermaxManager"

namespace UE::RivermaxCore
{
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxOutSendingFrameTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxOutFrameReadyTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxInStartingFrameTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxInReceivedFrameTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxInSelectedFrameTraceEvents;
	TMap<uint8, FString> FRivermaxTracingUtils::RmaxOutAncSendingFrameTraceEvents;

	// Environment variable used to specify where Rivermax library is located. This enables having an installed version different than the one used by Unreal
	static const FString RivermaxLibraryEnvironmentVariable = TEXT("RIVERMAX_PATH_1_90_18");
	static const uint32 RivermaxVersion[3] = { 1, 90, 18 };
}

namespace UE::RivermaxCore::Private
{
	static TAutoConsoleVariable<int32> CVarRivermaxEnableGPUDirectCapability(
		TEXT("Rivermax.GPUDirect"), 1,
		TEXT("Whether GPUDirect capability is validated at launch. Can be used to disable it entirely."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxEnableGPUDirectInputCapability(
		TEXT("Rivermax.GPUDirectInput"), 0,
		TEXT("Whether GPUDirect capability is enabled for input purposes. When enabled users need to make sure that GPU and DPU are on the same root complex.\n\
			When streaming multiple Rivermax Inputs using GPU Direct while GPU and DPU are on different root complexes, 2110 packets can be lost. "),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxEnableGPUDirectOutputCapability(
		TEXT("Rivermax.GPUDirectOutput"), 1,
		TEXT("Whether GPUDirect capability is enabled for output purposes."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxCoreAffinity(
		TEXT("Rivermax.CPUAffinity"), -1,
		TEXT("CPU Core to pin Rivermax output/input thread to. Negative means disabled. Core index is 0..X.\n\
			The best choice for the cpu core is one that is on the same NUMA node as the DPU/NIC used for Rivermax traffic."),
		ECVF_Default);


	static uint64_t RivermaxTimeHandler(void* Context)
	{
		IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>("RivermaxCore");
		if (RivermaxModule)
		{
			return RivermaxModule->GetRivermaxManager()->GetTime();
		}

		return 0;
	}

	FRivermaxManager::FRivermaxManager()
		: DeviceFinder(MakeUnique<FRivermaxDeviceFinder>())
	{
		// No need loading Cuda if we won't be looking at gpudirect capabilities
#if PLATFORM_WINDOWS
		if (CVarRivermaxEnableGPUDirectCapability.GetValueOnGameThread())
		{
			FModuleManager::LoadModuleChecked<FCUDAModule>("CUDA");
		}
#endif // PLATFORM_WINDOWS

		//Postpone initialization after all modules have been loaded
		InitHandle = FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FRivermaxManager::InitializeLibrary);
	}

	FRivermaxManager::~FRivermaxManager()
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.Remove(InitHandle);

		if (bIsCleanupRequired)
		{
			rmx_status Status = GetApi()->rmx_cleanup();
			if (Status != RMX_OK)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to cleanup Rivermax Library. Status: %d", Status);
			}
			UE_LOGF(LogRivermax, Log, "Rivermax Library has shutdown.");
		}

		if (LibraryHandle != nullptr)
		{
			FPlatformProcess::FreeDllHandle(LibraryHandle);
			LibraryHandle = nullptr;
		}
	}

	bool FRivermaxManager::IsManagerInitialized() const
	{
		return bIsInitialized;
	}

	bool FRivermaxManager::LoadRivermaxLibrary()
	{
#if defined(RIVERMAX_LIBRARY_PLATFORM_PATH) && defined(RIVERMAX_LIBRARY_NAME)

		FString LibraryPath = TEXT(UE_STRINGIZE(RIVERMAX_LIBRARY_PLATFORM_PATH));

		// Look for environment variable defining where the library version we are looking for is located

		FString EnvVarRivermaxPath = FPlatformMisc::GetEnvironmentVariable(*RivermaxLibraryEnvironmentVariable);
		if (!EnvVarRivermaxPath.IsEmpty())
		{
			if (FPaths::DirectoryExists(EnvVarRivermaxPath))
			{
				LibraryPath = EnvVarRivermaxPath;
			}
		}
		else
		{
#if PLATFORM_WINDOWS
			// On Windows, fall back to locating the library via the Mellanox WinOF2 registry key.
			FString MellanoxInstalledPath;
			static const TCHAR* MLXFolder = TEXT("SOFTWARE\\Mellanox\\MLNX_WinOF2");
			static const TCHAR* InstalledPathKey = TEXT("InstalledPath");
			const bool bFoundKey = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, MLXFolder, InstalledPathKey, MellanoxInstalledPath);
			if (bFoundKey)
			{
				// Mellanox installed path will point to OF2 folder. We need to get the parent of it since that's where Rivermax will get installed by default.
				const FString MellanoxRootDir = FPaths::Combine(MellanoxInstalledPath, TEXT("..\\Rivermax\\lib"));
				if (FPaths::DirectoryExists(MellanoxRootDir))
				{
					// Mellanox based Rivermax directory exists, replace default root dir
					LibraryPath = MellanoxRootDir;
				}
			}
#elif PLATFORM_LINUX
			// On Linux, the compile-time default path (RIVERMAX_LIBRARY_PLATFORM_PATH =
			// /usr/lib/x86_64-linux-gnu) set above is already correct. The Rivermax SDK
			// installs to a fixed system path; no additional lookup is needed.
#endif // PLATFORM_WINDOWS
		}

		const FString LibraryName = TEXT(UE_STRINGIZE(RIVERMAX_LIBRARY_NAME));
		if (FPaths::DirectoryExists(LibraryPath))
		{
			FPlatformProcess::PushDllDirectory(*LibraryPath);
			LibraryHandle = FPlatformProcess::GetDllHandle(*LibraryName);
			FPlatformProcess::PopDllDirectory(*LibraryPath);
		}
#if PLATFORM_LINUX
		// If the configured path doesn't contain the library (e.g. on RPM-based distros
		// where Rivermax installs to /usr/lib64 instead of /usr/lib/x86_64-linux-gnu),
		// fall back to the system library search path (LD_LIBRARY_PATH, ldconfig).
		if (LibraryHandle == nullptr)
		{
			LibraryHandle = FPlatformProcess::GetDllHandle(*LibraryName);
		}
#endif
#endif

		const bool bIsLibraryValid = LibraryHandle != nullptr;
		if(!bIsLibraryValid)
		{
			UE_LOGF(LogRivermax, Log, "Failed to load Rivermax library.");
		}
		else
		{
			UE_LOGF(LogRivermax, Display, "Successfully loaded Rivermax library from '%ls'", *LibraryPath);
		}

		return bIsLibraryValid;
	}

	uint64 FRivermaxManager::GetTime() const
	{
		uint64_t Time = 0;

		switch (GetTimeSource())
		{
			case ERivermaxTimeSource::PTP:
			{
				//Rivermax is usable and configured to read PTP
				if (GetApi()->rmx_get_time(RMX_TIME_PTP, &Time) != RMX_OK)
				{
					UE_LOGF(LogRivermax, Warning, "PTP time is the time source but was unavailable");
				}
				break;
			}
			case ERivermaxTimeSource::Engine:
			{
				Time = uint64(FPlatformTime::Seconds() * 1E9);
				break;
			}
			case ERivermaxTimeSource::System:
			{
				// Using chrono to get nanoseconds precision. 
				Time = (uint64_t)std::chrono::duration_cast<std::chrono::nanoseconds>((std::chrono::system_clock::now()).time_since_epoch()).count();
				break;
			}
			default:
			{
				checkNoEntry();
			}
		}

		return Time;
	}

	bool FRivermaxManager::LoadRivermaxFunctions()
	{
		memset(&FuncList, 0, sizeof(RIVERMAX_API_FUNCTION_LIST));
		bIsLibraryLoaded = UE::RivermaxCore::Private::LoadLibraryFunctions(&FuncList, LibraryHandle);

		if (!bIsLibraryLoaded)
		{
			UE_LOGF(LogRivermax, Log, "Failed to load Rivermax library functions.");
		}

		return bIsLibraryLoaded;
	}

	ERivermaxTimeSource FRivermaxManager::GetTimeSource() const
	{
		return TimeSource;
	}

	void FRivermaxManager::SetCPUAffinity(int32 CpuCore)
	{
		if (CpuCore < 0)
		{
			UE_LOGF(LogRivermax, Log, "Rivermax CPU affinity not set (CpuCore=%d)", CpuCore);
			return;
		}

		const int32 NumLogicalCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
		if (NumLogicalCores <= 0)
		{
			UE_LOGF(LogRivermax, Warning, "Rivermax CPU affinity not set: could not query core count (CpuCore=%d)", CpuCore);
			return;
		}

		if (CpuCore >= NumLogicalCores)
		{
			UE_LOGF(LogRivermax, Warning, "Rivermax CPU affinity not set: CpuCore=%d out of range [0..%d]", CpuCore, NumLogicalCores - 1);
			return;
		}

		constexpr int32 BitsPerWord = 64;
		const int32 NumBits = CpuCore + 1;
		const int32 NumWords = (NumBits + BitsPerWord - 1) / BitsPerWord;

		TArray<uint64_t> CpuMask;
		CpuMask.Init(0ull, NumWords);

		const int32 WordIndex = CpuCore / BitsPerWord;
		const int32 BitOffset = CpuCore % BitsPerWord;
		CpuMask[WordIndex] |= (1ULL << BitOffset);

		const rmx_status Status = GetApi()->rmx_set_cpu_affinity(CpuMask.GetData(), static_cast<size_t>(NumBits));

		if (Status != RMX_OK)
		{
			UE_LOGF(LogRivermax, Error, "Failed to set affinity to core %d (status=%d)", CpuCore, (int32)Status);
		}
		else
		{
			UE_LOGF(LogRivermax, Log, "Set CPU affinity to core %d", CpuCore);
		}

	}

	void FRivermaxManager::InitializeLibrary()
	{
		bool bCanProceed = FApp::CanEverRender();
		
		if (bCanProceed)
		{
			bCanProceed = VerifyPrerequesites();
		}

		if(bCanProceed)
		{
			bCanProceed = LoadRivermaxLibrary();
		}
		
		if (bCanProceed)
		{
			bCanProceed = LoadRivermaxFunctions();
		}

		if (bCanProceed)
		{
			// Load the specific version we have the API For
			const rmx_version Policy = 
			{
				RMX_VERSION_MAJOR,
				RMX_VERSION_MINOR,
				RMX_VERSION_PATCH
			};

			// Bug with 1.31 version where this is required to be called before _init() even though it is optional
			GetApi()->rmx_enable_system_signal_handling();

			const int32 Core = CVarRivermaxCoreAffinity.GetValueOnGameThread();
			SetCPUAffinity(Core);

			const rmx_status Status = GetApi()->_rmx_init(&Policy);
			if (Status == RMX_OK)
			{
				bIsCleanupRequired = true;

				const URivermaxSettings* Settings = GetDefault<URivermaxSettings>();
				TimeSource = Settings->TimeSource;
				bool bIsClockConfigured = InitializeClock(TimeSource);

				if (!bIsClockConfigured && TimeSource == ERivermaxTimeSource::PTP)
				{
					TimeSource = ERivermaxTimeSource::System;
					bIsClockConfigured = InitializeClock(TimeSource);
				}

				if (bIsClockConfigured)
				{
					const rmx_version* CurrentVersion = GetApi()->rmx_get_version_numbers();
					if (CurrentVersion)
					{
						if (CVarRivermaxEnableGPUDirectCapability.GetValueOnGameThread())
						{
							VerifyGPUDirectCapability();
						}

						InitializeTraceMarkupStrings();

						bIsLibraryInitialized = true;
						const TCHAR* GPUDirectSupport = bIsGPUDirectSupported ? TEXT("with GPUDirect support.") : TEXT("without GPUDirect support.");
						if (RivermaxVersion[0] != CurrentVersion->major || RivermaxVersion[1] != CurrentVersion->minor || RivermaxVersion[2] != CurrentVersion->patch)
						{
							UE_LOGF(LogRivermax, Warning, "Rivermax library version %d.%d.%d is not officially supported by this version of the engine.\n"
															"Please install version: %d.%d.%d or set the environment variable %ls to point to the lib folder of the Rivermax library.",
								CurrentVersion->major, CurrentVersion->minor, CurrentVersion->patch, 
								RivermaxVersion[0], RivermaxVersion[1], RivermaxVersion[2], 
								*RivermaxLibraryEnvironmentVariable);
							UE_LOGF(LogRivermax, Warning, "The engine will proceed with the current library version %ls However, there might be unexpected issues.", GPUDirectSupport);
						}
						else
						{
							UE_LOGF(LogRivermax, Log, "Rivermax library version %d.%d.%d successfully initialized, %ls", CurrentVersion->major, CurrentVersion->minor, CurrentVersion->patch, GPUDirectSupport);
						}
					}
					else
					{
						UE_LOGF(LogRivermax, Log, "Failed to retrieve Rivermax library version. Status: %d", Status);
					}
				}
			}
			else if (Status == RMX_LICENSE_ISSUE)
			{
				UE_LOGF(LogRivermax, Warning, "Rivermax License could not be found. Have you configured RIVERMAX_LICENSE_PATH environment variable?");
			}
			else if (Status > RMX_INVALID_PARAM_MIX && Status <= RMX_INVALID_PARAM_10)
			{
				UE_LOGF(LogRivermax, Warning, "Rivermax initialization error, installed Rivermax SDK must be version %d.%d.%d.", RMX_VERSION_MAJOR, RMX_VERSION_MINOR, RMX_VERSION_PATCH);
			}

			if (bIsLibraryInitialized == false)
			{
				UE_LOGF(LogRivermax, Error, "Rivermax library failed to initialize. Status: %d", Status);
			}
		}
		else
		{
			UE_LOGF(LogRivermax, Log, "Skipping Rivermax initialization. Library won't be usable.");
		}

		bIsInitialized = true;
		PostInitDelegate.Broadcast();
	}

	TConstArrayView<UE::RivermaxCore::FRivermaxDeviceInfo> FRivermaxManager::GetDevices() const
	{
		return DeviceFinder->GetDevices();
	}

	bool FRivermaxManager::GetMatchingDevice(const FString& InSourceIP, FString& OutDeviceIP) const
	{
		return DeviceFinder->ResolveIP(InSourceIP, OutDeviceIP);
	}

	bool FRivermaxManager::IsValidIP(const FString& InSourceIP) const
	{
		return DeviceFinder->IsValidIP(InSourceIP);
	}

	bool FRivermaxManager::IsGPUDirectSupported() const
	{
		return bIsGPUDirectSupported;
	}

	void FRivermaxManager::VerifyGPUDirectCapability()
	{
		bIsGPUDirectSupported = false;

#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();
		if (RHIType != ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogRivermax, Log, "GPUDirect won't be available for Rivermax as it's only available with D3D12 RHI.");
			return;
		}
		
		FCUDAModule* CudaModule = FModuleManager::GetModulePtr<FCUDAModule>("CUDA");
		if (CudaModule)
		{	
			CUcontext Context = CudaModule->GetCudaContext();
			if (Context == nullptr)
			{
				UE_LOGF(LogRivermax, Log, "GPUDirect won't be available for Rivermax since Cuda context can't be fetched.");
				return;
			}

			CudaModule->DriverAPI()->cuCtxPushCurrent(Context);

			int DeviceCount = -1;
			CUresult Result = CudaModule->DriverAPI()->cuDeviceGetCount(&DeviceCount);
			if (DeviceCount <= 0)
			{
				UE_LOGF(LogRivermax, Log, "No Cuda compatible device found. GPUDirect won't be available for Rivermax. Cuda status: %d", Result);
				return;
			}

			// todo: add support for mgpu compatibility verification
			const int GPUIndex = CudaModule->GetCudaDeviceIndex();
			CUdevice CudaDevice;
			Result = CudaModule->DriverAPI()->cuDeviceGet(&CudaDevice, GPUIndex);
			if (Result != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Log, "Could not get a valid Cuda device. GPUDirect won't be available for Rivermax. Cuda status: %d", Result);
				return;
			}


			int SupportsVMM = 0;
			Result  = CudaModule->DriverAPI()->cuDeviceGetAttribute(&SupportsVMM, CU_DEVICE_ATTRIBUTE_VIRTUAL_MEMORY_MANAGEMENT_SUPPORTED, CudaDevice);
			if (SupportsVMM != 1)
			{
				UE_LOGF(LogRivermax, Log, "Cuda device doesn't support virtual memory management. GPUDirect won't be available for Rivermax. Cuda status: %d", Result);
				return;
			}

			int SupportsRDMA = 0;
			Result = CudaModule->DriverAPI()->cuDeviceGetAttribute(&SupportsRDMA, CU_DEVICE_ATTRIBUTE_GPU_DIRECT_RDMA_WITH_CUDA_VMM_SUPPORTED, CudaDevice);
			if (SupportsRDMA != 1)
			{
				UE_LOGF(LogRivermax, Log, "Cuda device doesn't support RDMA. GPUDirect won't be available for Rivermax. Cuda status: %d", Result);
				return;
			}

			bIsGPUDirectSupported = true;
		}
#else
		UE_LOG(LogRivermax, Log, TEXT("GPUDirect is not supported on this platform."));
#endif // PLATFORM_WINDOWS
	}

	bool FRivermaxManager::InitializeClock(ERivermaxTimeSource DesiredTimeSource)
	{
		switch (DesiredTimeSource)
		{
			case ERivermaxTimeSource::PTP:
			{
				FString PTPAddress;
				const URivermaxSettings* Settings = GetDefault<URivermaxSettings>();
				if (DeviceFinder->ResolveIP(Settings->PTPInterfaceAddress, PTPAddress))
				{
					in_addr DeviceIPAddr;
					if (inet_pton(AF_INET, StringCast<ANSICHAR>(*PTPAddress).Get(), &DeviceIPAddr))
					{
						UE_LOGF(LogRivermax, Display, "Configure Rivermax clock for PTP using device interface %ls", *PTPAddress);
						rmx_device_iface DeviceInterface;
						rmx_ip_addr RmaxDeviceAddr;
						RmaxDeviceAddr.family = AF_INET;
						RmaxDeviceAddr.addr.ipv4 = DeviceIPAddr;
						rmx_status Status = GetApi()->rmx_retrieve_device_iface(&DeviceInterface, &RmaxDeviceAddr);
						if (Status == RMX_OK)
						{
							rmx_ptp_clock_params PTPClockParameters;
							GetApi()->rmx_init_ptp_clock(&PTPClockParameters);
							GetApi()->rmx_set_ptp_clock_device(&PTPClockParameters, &DeviceInterface);
							Status = GetApi()->rmx_use_ptp_clock(&PTPClockParameters);
							if (Status == RMX_OK)
							{
								return true;
							}
							else if (Status == RMX_CLOCK_TYPE_NOT_SUPPORTED)
							{
								UE_LOGF(LogRivermax, Warning, "PTP clock not supported on device interface %ls. Falling back to system clock.", *PTPAddress);
							}
							else
							{
								UE_LOGF(LogRivermax, Warning, "Failed to configure PTP clock on device interface %ls with error '%d'. Falling back to system clock.", *PTPAddress, Status);
							}
						}
						else
						{
							UE_LOGF(LogRivermax, Warning, "Failed to get device with IP %ls during PTP clock configuration, Status: '%d'. Falling back to system clock.", *PTPAddress, Status);
						}

						return false;
					}
				}

				UE_LOGF(LogRivermax, Warning, "Failed to configure PTP clock. Device interface %ls was invalid. Falling back to system clock.", *PTPAddress);
				return false;
			}
			case ERivermaxTimeSource::Engine:
			{
				rmx_user_clock_params UserClockParameters;
				FuncList.rmx_init_user_clock(&UserClockParameters);
				FuncList.rmx_set_user_clock_handler(&UserClockParameters, RivermaxTimeHandler);

				UE_LOGF(LogRivermax, Display, "Configure Rivermax clock for engine time");
				const rmx_status Status = FuncList.rmx_use_user_clock(&UserClockParameters);
				if (Status != RMX_OK)
				{
					UE_LOGF(LogRivermax, Warning, "Failed to configure clock to use engine time with error '%d'.", Status);
				}

				return Status == RMX_OK;
			}
			case ERivermaxTimeSource::System:
			{
				// System clock is the default Rivermax clock if none others are used.
				UE_LOGF(LogRivermax, Display, "Configure Rivermax clock for system time");
				return true;
			}
			default:
			{
				checkNoEntry();
				return false;
			}
		}
	}

	bool FRivermaxManager::IsLibraryInitialized() const
	{
		return bIsLibraryInitialized;
	}
	
	bool FRivermaxManager::ValidateLibraryIsLoaded() const
	{
		static const FText ErrorText = LOCTEXT("RivermaxLibraryError", "Rivermax library was not initialized correctly. Did you install the Rivermax SDK?");

		if (!bIsLibraryInitialized)
		{
#if WITH_EDITOR
			if (GEditor)
			{
				FNotificationInfo Info{ErrorText};
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
#endif
			UE_LOGF(LogRivermax, Warning, "%ls", *ErrorText.ToString());
		}
		
		return bIsLibraryInitialized;
	}

	const UE::RivermaxCore::Private::RIVERMAX_API_FUNCTION_LIST* FRivermaxManager::GetApi() const
	{
		if (bIsLibraryLoaded)
		{
			return &FuncList;
		}

		return nullptr;
	}

	FOnPostRivermaxManagerInit& FRivermaxManager::OnPostRivermaxManagerInit()
	{
		return PostInitDelegate;
	}

	bool FRivermaxManager::VerifyPrerequesites()
	{
		bool bSuccess = false;

#if PLATFORM_WINDOWS
		// This registry key is required in order to use rivermax sdk. This puts the network adapter in rivermax mode.
		// With 1.20, SDK currently crashes if that is not present / set. Avoid loading the library if that's the case and notify the user.
		FString RivermaxSetupValue;
		static const TCHAR* MLXFolder = TEXT("SYSTEM\\CurrentControlSet\\Services\\mlx5\\Parameters");
		static const TCHAR* RivermaxKey = TEXT("RivermaxSetup");
		const bool bFoundKey = FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, MLXFolder, RivermaxKey, RivermaxSetupValue);
		if (bFoundKey)
		{
			int32 KeyValue = 0;
			LexFromString(KeyValue, *RivermaxSetupValue);
			if (KeyValue == 1)
			{
				bSuccess = true;
				UE_LOGF(LogRivermax, Verbose, "Rivermax setup registry key was found and enabled.");
			}
			else
			{
				UE_LOGF(LogRivermax, Warning, "Rivermax setup registry key was found but not enabled.");
			}
		}
		else
		{
			UE_LOGF(LogRivermax, Warning, "Could not find RivermaxSetup registry key. Verify installation and reboot your machine.");
		}
#elif PLATFORM_LINUX
		// On Linux, verify the mlx5 kernel module is loaded as a proxy for driver installation.
		// Full adapter mode configuration (mlxconfig INTERNAL_CPU_MODEL) is the user's responsibility
		// and cannot be checked programmatically without external tools.
		if (access("/sys/module/mlx5_core", F_OK) == 0)
		{
			bSuccess = true;
		}
		else
		{
			UE_LOG(LogRivermax, Warning, TEXT("mlx5_core kernel module is not loaded. Verify Mellanox/NVIDIA OFED driver installation."));
		}
#endif // PLATFORM_WINDOWS

		if (bSuccess)
		{
			// Crash was reported calling rmax_init with all prerequesite installation correctly done but no card installed. 
			// This will prevent us from going forward with rivermax initialization
			bSuccess = DeviceFinder->GetDevices().Num() > 0;
			if (!bSuccess)
			{
				UE_LOGF(LogRivermax, Warning, "Could not find any compatible Rivermax network adapters.");
			}
		}

		return bSuccess;
	}

	bool FRivermaxManager::IsGPUDirectInputSupported() const
	{
		return bIsGPUDirectSupported && CVarRivermaxEnableGPUDirectInputCapability.GetValueOnAnyThread();
	}

	bool FRivermaxManager::IsGPUDirectOutputSupported() const
	{
		return bIsGPUDirectSupported && CVarRivermaxEnableGPUDirectOutputCapability.GetValueOnAnyThread();
	}

	bool FRivermaxManager::EnableDynamicHeaderSupport(const FString& Interface)
	{
		if(uint32* Users = DynamicHeaderUsers.Find(Interface))
		{
			(*Users)++;
			return true;
		}

		sockaddr_in RivermaxDevice;
		FMemory::Memset(&RivermaxDevice, 0, sizeof(RivermaxDevice));
		if (inet_pton(AF_INET, StringCast<ANSICHAR>(*Interface).Get(), &RivermaxDevice.sin_addr) != 1)
		{
			UE_LOGF(LogRivermax, Warning, "Failed to use inet_pton for interface '%ls'", *Interface);
			return false;
		}

		rmx_device_iface DeviceInterface;
		rmx_ip_addr RmaxDeviceAddr;
		RmaxDeviceAddr.family = AF_INET;
		RmaxDeviceAddr.addr.ipv4 = RivermaxDevice.sin_addr;
		rmx_status Status = GetApi()->rmx_retrieve_device_iface(&DeviceInterface, &RmaxDeviceAddr);
		if (Status != RMX_OK)
		{
			UE_LOGF(LogRivermax, Warning, "Failed to get device for interface '%ls'", *Interface);
			return false;
		}

		rmx_device_capabilities Capabilities;
		rmx_clear_device_capabilities_enquiry(&Capabilities);
		rmx_mark_device_capability_for_enquiry(&Capabilities, RMX_DEVICE_CAP_RTP_DYNAMIC_HDS);
		Status = GetApi()->rmx_enquire_device_capabilities(&DeviceInterface, &Capabilities);
		if (Status != RMX_OK) 
		{
			UE_LOGF(LogRivermax, Warning, "Failed to query capabilities for device IP '%ls'", *Interface);
			return false;
		}

		const bool bIsDynamicHeaderSplitSupported = rmx_is_device_capability_supported(&Capabilities, RMX_DEVICE_CAP_RTP_DYNAMIC_HDS);
		if (!bIsDynamicHeaderSplitSupported) 
		{
			UE_LOGF(LogRivermax, Warning, "RTP dynamic header data split is not supported for device IP '%ls'.\nMake sure firmware is updated and FLEX_PARSER is configured according to deployment guide.", *Interface)
			return false;
		}

		rmx_device_config DeviceConfiguration;
		rmx_clear_device_config_attributes(&DeviceConfiguration);
		constexpr rmx_device_config_attribute Attribute = RMX_DEVICE_CONFIG_RTP_SMPTE_2110_20_DYNAMIC_HDS;
		rmx_set_device_config_attribute(&DeviceConfiguration, Attribute);
		
		Status = GetApi()->rmx_apply_device_config(&DeviceInterface, &DeviceConfiguration);
		if (Status != RMX_OK)
		{
			UE_LOGF(LogRivermax, Warning, "Could not enable dynamic header split for device IP '%ls'. Error: %d", *Interface, Status);
			return false;
		}

		UE_LOGF(LogRivermax, Log, "Dynamic header split for device IP '%ls' has been enabled.", *Interface);
		DynamicHeaderUsers.FindOrAdd(Interface, 1);
		return true;
	}

	void FRivermaxManager::DisableDynamicHeaderSupport(const FString& Interface)
	{
		if (uint32* Users = DynamicHeaderUsers.Find(Interface))
		{
			(*Users)--;
			if((*Users) > 0)
			{
				return;
			}

			bool bCanUnsetDevice = true;
			sockaddr_in DeviceToUse;
			FMemory::Memset(&DeviceToUse, 0, sizeof(DeviceToUse));
			if (inet_pton(AF_INET, StringCast<ANSICHAR>(*Interface).Get(), &DeviceToUse.sin_addr) != 1)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to use inet_pton for interface '%ls'", *Interface);
				bCanUnsetDevice = false;
			}

			rmx_device_iface DeviceInterface;
			if (bCanUnsetDevice)
			{
				rmx_ip_addr RmaxDeviceAddr;
				RmaxDeviceAddr.family = AF_INET;
				RmaxDeviceAddr.addr.ipv4 = DeviceToUse.sin_addr;
				rmx_status Status = GetApi()->rmx_retrieve_device_iface(&DeviceInterface, &RmaxDeviceAddr);
				if (Status != RMX_OK)
				{
					UE_LOGF(LogRivermax, Warning, "Failed to get device for interface '%ls'", *Interface);
					bCanUnsetDevice = false;
				}
			}

			if (bCanUnsetDevice)
			{

				rmx_device_config DeviceConfiguration;
				rmx_clear_device_config_attributes(&DeviceConfiguration);
				constexpr rmx_device_config_attribute Attribute = RMX_DEVICE_CONFIG_RTP_SMPTE_2110_20_DYNAMIC_HDS;
				rmx_set_device_config_attribute(&DeviceConfiguration, Attribute);

				const rmx_status Status = GetApi()->rmx_revert_device_config(&DeviceInterface, &DeviceConfiguration);
				if (Status != RMX_OK)
				{
					UE_LOGF(LogRivermax, Warning, "Could not disable dynamic header split for device IP '%ls'. Error: %d", *Interface, Status);
				}
				else
				{
					UE_LOGF(LogRivermax, Log, "Dynamic header split for device IP '%ls' has been disabled.", *Interface);
				}
			}

			DynamicHeaderUsers.Remove(Interface);
		}
	}

	void FRivermaxManager::InitializeTraceMarkupStrings()
	{
		constexpr uint8 MarkupCount = 10;

		for (uint8 Index = 0; Index < MarkupCount; ++Index)
		{
			FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents.Add(Index, FString::Printf(TEXT("MediaCapturePipe: %u"), Index));
			FRivermaxTracingUtils::RmaxOutSendingFrameTraceEvents.Add(Index, FString::Printf(TEXT("RmaxOut Sending: %u"), Index));
			FRivermaxTracingUtils::RmaxOutFrameReadyTraceEvents.Add(Index, FString::Printf(TEXT("RmaxOut FrameReady: %u"), Index));
			FRivermaxTracingUtils::RmaxInStartingFrameTraceEvents.Add(Index, FString::Printf(TEXT("RmaxIn Starting: %u"), Index));
			FRivermaxTracingUtils::RmaxInReceivedFrameTraceEvents.Add(Index, FString::Printf(TEXT("RmaxIn Received: %u"), Index));
			FRivermaxTracingUtils::RmaxInSelectedFrameTraceEvents.Add(Index, FString::Printf(TEXT("RmaxIn Selected: %u"), Index));
			FRivermaxTracingUtils::RmaxOutAncSendingFrameTraceEvents.Add(Index, FString::Printf(TEXT("RmaxOut ANC Sending: %u"), Index));
		}
	}

}

#undef LOCTEXT_NAMESPACE // RivermaxManager

