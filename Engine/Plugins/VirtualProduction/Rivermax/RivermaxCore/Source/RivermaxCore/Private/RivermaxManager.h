// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxManager.h"
#include "RivermaxWrapper.h"

namespace UE::RivermaxCore::Private
{
	class FRivermaxDeviceFinder;
	
	class FRivermaxManager : public UE::RivermaxCore::IRivermaxManager
	{
	public:
		FRivermaxManager();
		~FRivermaxManager();

	public:
		//~ Begin IRivermaxManager interface
		virtual bool IsManagerInitialized() const override;
		virtual bool IsLibraryInitialized() const override;
		virtual bool ValidateLibraryIsLoaded() const override;
		virtual const RIVERMAX_API_FUNCTION_LIST* GetApi() const override;
		virtual FOnPostRivermaxManagerInit& OnPostRivermaxManagerInit() override;
		virtual uint64 GetTime() const override;
		virtual ERivermaxTimeSource GetTimeSource() const override;
		virtual TConstArrayView<FRivermaxDeviceInfo> GetDevices() const override;
		virtual bool GetMatchingDevice(const FString& InSourceIP, FString& OutDeviceIP) const override;
		virtual bool IsValidIP(const FString& InSourceIP) const override;
		virtual bool IsGPUDirectSupported() const override;
		virtual bool IsGPUDirectInputSupported() const override;
		virtual bool IsGPUDirectOutputSupported() const override;
		virtual bool EnableDynamicHeaderSupport(const FString& Interface) override;
		virtual void DisableDynamicHeaderSupport(const FString& Interface) override;
		//~ End IRivermaxManager interface

	private:

		/** Loads the rivermax dll shipping with engine */
		bool LoadRivermaxLibrary();

		/** Load function pointers from the DLL. */
		bool LoadRivermaxFunctions();
		
		/** Initializes rivermax library */
		void InitializeLibrary();

		/** Sets CPU affinity for Rivermax*/
		void SetCPUAffinity(int32 CpuCore);

		/** Verify if system is capable of doing GPUDirect */
		void VerifyGPUDirectCapability();

		/** Initializes clock used by Rivermax internals */
		bool InitializeClock(ERivermaxTimeSource DesiredTimeSource);

		/** Verify if all prerequesites to go through Rivermax initialization have been met */
		bool VerifyPrerequesites();

		/** Initializes maps of strings used in various trace events to avoid allocating them continuously */
		void InitializeTraceMarkupStrings();
	private:

		/** True when manager has been initialized */
		bool bIsInitialized = false;

		/** True when library is usable */
		bool bIsLibraryInitialized = false;

		/** True when library function pointers are available. */
		bool bIsLibraryLoaded = false;

		/** Whether library was initialized but failed along the way and needs to be cleaned up on exit */
		bool bIsCleanupRequired = false;

		/** Handle pointer to the loaded library */
		void* LibraryHandle = nullptr;

		/** Whether GPU direct is supported globally for rivermax. Currently means Cuda is present with RDMA support */
		bool bIsGPUDirectSupported = false;

		/** Current time source */
		ERivermaxTimeSource TimeSource = ERivermaxTimeSource::PTP;

		/** Rivermax device finder listing all usable interfaces */
		TUniquePtr<FRivermaxDeviceFinder> DeviceFinder;

		/** Delegate triggered when initialization has completed */
		FOnPostRivermaxManagerInit PostInitDelegate;

		/** Streams using dynamic header size per users. When last one is removing itself, it will be disabled */
		TMap<FString, uint32> DynamicHeaderUsers;

		/** List of function pointers to the rivermax DLL. */
		RIVERMAX_API_FUNCTION_LIST FuncList;

		/** Delegate handle. Used to remove the delegate from subscribed module initialization delegate collection. */
		FDelegateHandle InitHandle;
	};
}


