// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxOutputFrame.h"

namespace UE::RivermaxCore::Private
{
	FRivermaxOutputFrame::FRivermaxOutputFrame()
		: Buffer(nullptr)
	{
		Clear();
	}

	void FRivermaxOutputFrame::Clear()
	{
		PacketCounter = 0;
		LineNumber = 0;
	}

	void FRivermaxOutputFrame::Reset()
	{
		bCaughtTimingIssue = false;
		Clear();
	}

}
