// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Math/NumericLimits.h"
#include "Misc/Timecode.h"
#include "MediaObjectPool.h"

namespace UE::RivermaxCore::Private
{
	/**
	 * Output frame descriptor. Contains data to be sent on the wire and packetization tracking
	 */
	class FRivermaxOutputFrame : public IMediaPoolable
	{
	public:
		FRivermaxOutputFrame();
		FRivermaxOutputFrame(const FRivermaxOutputFrame&) = delete;
		FRivermaxOutputFrame& operator=(const FRivermaxOutputFrame&) = delete;

		/** Reset internal counters to make it resendable */
		void Clear();

		/** Reset identifier and internal counters to make it available */
		void Reset();

		/** 
		 * Buffer where we copy our data.
		 * 
		 * In Case of Video stream this is where we copy texture to be sent out 
		 * If using GPUDirect, memory will be allocated in cuda space
		 * 
		 * In case of ANCILLARY this will contain our ancillary 
		 */
		void* Buffer = nullptr;

		/** Timecode at which this frame was captured */
		FTimecode Timecode;

		/** Variables about progress of packetization of the frame */
		uint32 PacketCounter = 0;
		uint32 LineNumber = 0;
		uint16 SRDOffset = 0;
		uint32 ChunkNumber = 0;

		/** Timestamp of this frame used for RTP headers  */
		uint32 MediaTimestamp = 0;

		/** Payload (data) pointer retrieved from Rivermax for the next chunk */
		void* PayloadPtr = nullptr;
		
		/** Header pointer retrieved from Rivermax for the next chunk */
		void* HeaderPtr = nullptr;
		
		/** Cached address of beginning of frame in Rivermax's memblock. Used when using intermediate buffer/ */
		void* FrameStartPtr = nullptr;
		
		/** Offset in the frame where we are at to copy next block of data */
		SIZE_T Offset = 0;

		/** Whether timing issues were detected while sending frame out. If yes, we skip the next frame boundary */
		bool bCaughtTimingIssue = false;

	public:
		/**
		* Set frame counter .
		*/
		void SetFrameCounter(uint64 InFrameCounter)
		{
			FrameCounter = InFrameCounter;
		}

		/**
		* Returns frame counter corresponding to the data captured by Media Capture.
		*/
		uint64 GetFrameCounter()
		{
			return FrameCounter;
		}

		/**
		 * Used to check if returned object is ready for reuse right away
		 */
		virtual bool IsReadyForReuse()
		{
			return true;
		}

		/**
		 * Used to check if returned object is ready for reuse right away
		 */
		virtual void ShutdownPoolable() {}

	private:
		/** Number corresponding to the engine frame counter of the frame captured by Media Capture.*/
		uint64 FrameCounter;
	};
}


