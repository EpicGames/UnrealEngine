// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxPTPUtils.h"

#include "Misc/FrameRate.h"
#include "RivermaxTypes.h"

namespace UE::RivermaxCore
{
	uint64 GetNextAlignmentPoint(const uint64 InPTPTimeNanosec, const FFrameRate& InRate)
	{
		// Based on ST2059. Formulae from PTP time to next alignment point
		// NextAlignmentPoint = Floor( (TimeNs / IntervalNs + 1) * IntervalNs)
		// Time involved don't play well with double. Need to keep fixed point as much as possible

		// First part of formulae is to get the associated frame number of current time
		const uint64 CurrentFrameNumber = GetFrameNumber(InPTPTimeNanosec, InRate);

		// We want to get the time of the following frame number
		return GetAlignmentPointFromFrameNumber(CurrentFrameNumber + 1, InRate);
	}

	uint64 GetAlignmentPointFromFrameNumber(const uint64 InFrameNumber, const FFrameRate& InRate)
	{
		// Based on ST2059. Formulae from PTP time to next alignment point
		// NextAlignmentPoint = Floor( (CurrentFrameNumber + 1) * IntervalNs)

		const uint64 Nanoscale = 1E9;

		const uint64 NextAlignment = (InFrameNumber / InRate.Numerator) * InRate.Denominator;
		const uint64 NextAlignmentRemainderInterm = (InFrameNumber % InRate.Numerator) * InRate.Denominator;
		const uint64 NextAlignmentRemainder = (NextAlignmentRemainderInterm * Nanoscale) / InRate.Numerator;

		// We ceil to next nanosecond in case of any remainding. i.e 333333.33333 nanosec => 333334
		const uint64 OneNanoUp = (NextAlignmentRemainderInterm * Nanoscale) % InRate.Numerator > 0 ? 1 : 0;
		return (NextAlignment * Nanoscale) + NextAlignmentRemainder + OneNanoUp;
	}

	uint64 GetFrameNumber(const uint64 InPTPTimeNanosec, const FFrameRate& InRate)
	{
		const uint64 NanoScale = 1E9;

		// Split incoming time in nanoseconds in two parts : seconds and nanoseconds
		// Even after dividing by frame rate denominator (i.e 1001), we might overflow uint64 when multiplying by frame rate numerator (i.e. 24000)
		const uint64 PTPSeconds = InPTPTimeNanosec / NanoScale;
		const uint64 PTPNanoSeconds = InPTPTimeNanosec % NanoScale;

		// Truncated frames from the seconds portion
		const uint64 FSec = (PTPSeconds * InRate.Numerator) / InRate.Denominator;

		// We're missing the fractional frames from the seconds portion
		// ((PTPSeconds * InRate.Numerator) % InRate.Denominator) / InRate.Denominator;

		// Truncated frames from the nanoseconds portion
		const uint64 FNanoSec = (PTPNanoSeconds * InRate.Numerator) / (InRate.Denominator * NanoScale);

		// We're missing the fractional frames from the nanoseconds portion
		// ((PTPNanoSeconds * InRate.Numerator) % (InRate.Denominator * NanoScale)) / ((InRate.Denominator * NanoScale))
		
		// Add the fractional frames from both potions
		uint64 FractionalFrames = NanoScale * ((PTPSeconds * InRate.Numerator) % InRate.Denominator);
		FractionalFrames += (PTPNanoSeconds * InRate.Numerator) % (InRate.Denominator * NanoScale);
		FractionalFrames = FractionalFrames / (InRate.Denominator * NanoScale);

		return FSec + FNanoSec + FractionalFrames;
	}
}