// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxSettings.h"


FName URivermaxSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText URivermaxSettings::GetSectionText() const
{
	return NSLOCTEXT("RivermaxCorePlugin", "RivermaxSettingsSection", "NVIDIA Rivermax");
}
#endif //WITH_EDITOR
