// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxTypes.h"

#include "RivermaxUtils.h"

namespace UE::RivermaxCore
{
	FRivermaxAncOutputOptions::FRivermaxAncOutputOptions(uint8 InDID, uint8 InSDID)
	{
		using namespace UE::RivermaxCore::Private::Utils;
		DID = MakeDataIdentificationWord(InDID);
		SDID = MakeDataIdentificationWord(InSDID);
		DIDRaw = InDID;
		SDIDRaw = InSDID;
		Port = 50010;
	}
}