// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxUtils.h"

#include "HAL/IConsoleManager.h"
#include "RivermaxLog.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxTypes.h"
#include "RTPHeader.h"

#if RIVERMAX_PACKET_DEBUG
#include "Serialization/JsonSerializer.h"
#include "Misc/FileHelper.h"
#endif

namespace UE::RivermaxCore::Private::Utils
{
	uint32 TimestampToFrameNumber(uint32 Timestamp, const FFrameRate& FrameRate)
	{
		using namespace UE::RivermaxCore::Private::Utils;
		const double MediaFrameTime = Timestamp / MediaClockSampleRate;
		const uint32 FrameNumber = FMath::RoundToInt(MediaFrameTime * FrameRate.AsDecimal());
		return FrameNumber;
	}

	/** Returns a mediaclock timestamp, for rtp, based on a clock time */
	uint32 GetTimestampFromTime(uint64 InTimeNanosec, double InMediaClockRate)
	{
		// RTP timestamp is 32 bits and based on media clock (usually 90kHz).
		// Conversion based on rivermax samples

		const uint64 Nanoscale = 1E9;
		const uint64 Seconds = InTimeNanosec / Nanoscale;
		const uint64 Nanoseconds = InTimeNanosec % Nanoscale;
		const uint64 MediaFrameNumber = Seconds * InMediaClockRate;
		const uint64 MediaSubFrameNumber = Nanoseconds * InMediaClockRate / Nanoscale;
		const double Mask = 0x100000000;
		const double MediaTime = FMath::Fmod(MediaFrameNumber, Mask);
		const double MediaSubTime = FMath::Fmod(MediaSubFrameNumber, Mask);
		return MediaTime + MediaSubTime;
	}

	FTimecode GetTimecodeFromTime(uint64 InTimeNanosec, double InMediaClockRate, const FFrameRate& FrameRate)
	{
		FTimespan Timespan = FTimespan(InTimeNanosec / ETimespan::NanosecondsPerTick);

		// This should come from timecode provider or media profile.
		constexpr int32 DaylightSavingTimeHourOffset = 0; 
		constexpr int32 UTCSecondsOffset = 37;

		// Adjust for daylight saving that might be required. 
		Timespan -= FTimespan(DaylightSavingTimeHourOffset, 0, 0);

		// Convert from TAI PTP Time to UTC
		Timespan -= FTimespan(0, 0, UTCSecondsOffset);

		constexpr bool bRollOver = true;

		return FTimecode::FromTimespan(Timespan, FrameRate, bRollOver);
	}


	TArray<uint16> TimecodeToAtcUDW10(const FTimecode& InTimecode, const FFrameRate& Rate)
	{
		// UDW1-8 - Timecode.
		// UDW9-16 - Control, user bits.

		auto ToBCD = [](uint8 Value) -> uint8
		{
			return uint8(((Value / 10) << 4) | (Value % 10));
		};

		uint8 BCDFrames  = ToBCD(uint8(InTimecode.Frames));
		uint8 BCDSeconds = ToBCD(uint8(InTimecode.Seconds));
		uint8 BCDMinutes = ToBCD(uint8(InTimecode.Minutes));
		uint8 BCDHours   = ToBCD(uint8(InTimecode.Hours));

		// Each ATC User Data Word word follows the format: 
		// 
		// b0 - b2 = 0
		// b3 = DistributedBinaryBits (DBB) flag bit (may carry control info)
		// b4 - b7 = payload nibble
		// b8 = parity
		// b9 = inverse of b8
		

		// ExtraMask - to allow DF/CF flags in b6/b7 when needed
		auto MakeUDW10 = [](uint8 Nibble, bool DistributedBinaryBits = false, uint8 ExtraMask = 0) -> uint16
		{
			const uint16 Low8 = uint16(((Nibble & 0xF) << 4)
				| (DistributedBinaryBits ? 0x8 : 0x0)
				| (ExtraMask & 0xF0));

			const uint16 ParityEvenBit = (FMath::CountBits(static_cast<uint32>(Low8)) & 1) ? 1u : 0u;
			const uint16 Bit9 = (~ParityEvenBit) & 0x1;
			return static_cast<uint16>(Low8 | (ParityEvenBit << 8) | (Bit9 << 9));
		};

		TArray<uint16> UserDataWords;
		UserDataWords.Reserve(16);

		const FFrameRate TwentyNineNineSeven = FFrameRate(30000, 1001);
		const FFrameRate FiftyNineNintyFour = FFrameRate(60000, 1001);
		const bool bDropFrame = InTimecode.bDropFrameFormat || 
			((Rate == TwentyNineNineSeven) || (Rate == FiftyNineNintyFour));

		// Not set. Only used for composite analog NTSC/PAL gear.
		constexpr bool bColorFrame = false;

		{
			// UDW1: Frames units
			UserDataWords.Add(MakeUDW10((BCDFrames >> 0) & 0xF, /*DBB*/ false));

			// UDW2: User bits group 1 (unused)
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			uint8 NibTensF = (BCDFrames >> 4) & 0xF;
			uint8 Flags = 0;
			if (bDropFrame) Flags |= 0x40;          // DF -> b6
			if (bColorFrame) Flags |= 0x80;         // CF -> b7

			// UDW3: Frames tens + DF (b6) + CF (b7)
			UserDataWords.Add(MakeUDW10(NibTensF, false, Flags));

			// UDW4: User bits group 2 - Unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW5: Seconds units
			UserDataWords.Add(MakeUDW10((BCDSeconds >> 0) & 0xF, false));

			// UDW6: User bits group 3 - Unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW7: Seconds tens (optional flag at b7 is unused)
			UserDataWords.Add(MakeUDW10((BCDSeconds >> 4) & 0xF, false /*DBB*/, 0 /*no flag*/));

			// UDW8: User bits group 4 - unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW9: Minutes units
			UserDataWords.Add(MakeUDW10((BCDMinutes >> 0) & 0xF, false));

			// UDW10: User bits group 5 - unused
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW11: Minutes tens (+ optional flag at b7 - unused)
			UserDataWords.Add(MakeUDW10((BCDMinutes >> 4) & 0xF, false, 0));

			// UDW12: User bits group 6 - unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW13: Hours units
			UserDataWords.Add(MakeUDW10((BCDHours >> 0) & 0xF, false));

			// UDW14: User bits group 7 - unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}

		{
			// UDW15: Hours tens (+ optional flags b6/b7 - unused)
			UserDataWords.Add(MakeUDW10((BCDHours >> 4) & 0xF, false, 0));

			// UDW16: User bits group 8 - unused.
			UserDataWords.Add(MakeUDW10(0, false));
		}


		constexpr uint16 MaxNumUDW = 255;

		check(UserDataWords.Num() <= MaxNumUDW);

		return UserDataWords;
	}

	void GenerateRtpHeaderWithFields(const FString& JsonFilePath, const FString& OutputJsonPath)
	{
#if RIVERMAX_PACKET_DEBUG
		constexpr int32 MaxSize = 200;
		uint8 MemoryToDelete[MaxSize];
		FBigEndianHeaderPacker HeaderPacker(MemoryToDelete, MaxSize, true /*bClearExistingData*/);
		FString JsonContent;

		if (!FFileHelper::LoadFileToString(JsonContent, *JsonFilePath))
		{
			UE_LOGF(LogTemp, Error, "Failed to load json: %ls", *JsonFilePath);
			return;
		}

		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonContent);

		if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
		{
			UE_LOGF(LogTemp, Error, "Failed to parse json");
			return;
		}

		const TArray<TSharedPtr<FJsonValue>>* FieldArray;
		if (!RootObject->TryGetArrayField(TEXT("Fields"), FieldArray))
		{
			UE_LOGF(LogTemp, Error, "Json does not contain \"Fields\" array");
			return;
		}

		for (const TSharedPtr<FJsonValue>& FieldValue : *FieldArray)
		{
			const TSharedPtr<FJsonObject> FieldObj = FieldValue->AsObject();
			if (!FieldObj.IsValid()) continue;

			FString FieldName;
			int32 Bits = 0;

			if (!FieldObj->TryGetStringField(TEXT("Field"), FieldName))
			{
				UE_LOGF(LogTemp, Warning, "Missing member: \"Field\"");
				continue;
			}
			if (!FieldObj->TryGetNumberField(TEXT("Bits"), Bits))
			{
				UE_LOGF(LogTemp, Warning, "Missing member: \"Bits\" for field %ls", *FieldName);
				continue;
			}

			HeaderPacker.AddField(0, Bits, *FieldName);
		}
		HeaderPacker.Finalize();

		TArray<TSharedPtr<FJsonValue>> OutputFields;

		const FBigEndianHeaderPacker::FFieldInfo* PreviousBitField = nullptr;

		FBigEndianHeaderPacker::FFieldInfo Record;
		Record.Data = 0;
		Record.NumBits = 0;
		Record.FieldName = "";

		const TArray<TArray<FBigEndianHeaderPacker::FFieldInfo>>& FieldsInOrder = HeaderPacker.GetFieldsInOrderByRef();
		for (const TArray<FBigEndianHeaderPacker::FFieldInfo>& FieldsPerByte : FieldsInOrder)
		{
			for (int32 FieldIndex = FieldsPerByte.Num() - 1; FieldIndex >= 0; --FieldIndex)
			{
				const FBigEndianHeaderPacker::FFieldInfo* CurrentBitField = &FieldsPerByte[FieldIndex];

				// This will ignore the first field since to write the field into json file we need to be sure that previous field
				// isn't the same as current to group them up.
				if (PreviousBitField != nullptr)
				{
					TSharedPtr<FJsonObject> JsonField = MakeShared<FJsonObject>();
					// In case where previous field matches the current field, we group them up and add the bits together
					Record.NumBits += PreviousBitField->NumBits;

					// If previous field doesn't match the current, then we write it into json and null the bits.
					if (PreviousBitField->FieldName != CurrentBitField->FieldName)
					{
						JsonField->SetStringField(TEXT("Field"), PreviousBitField->FieldName);
						JsonField->SetNumberField(TEXT("Bits"), Record.NumBits);
						OutputFields.Add(MakeShared<FJsonValueObject>(JsonField));
						Record.NumBits = 0;
					}
				}

				PreviousBitField = CurrentBitField;

			}
		}

		// Process the last bit field.
		if (PreviousBitField != nullptr)
		{
			Record.NumBits += PreviousBitField->NumBits;
			TSharedPtr<FJsonObject> JsonField = MakeShared<FJsonObject>();
			JsonField->SetStringField(TEXT("Field"), PreviousBitField->FieldName);
			JsonField->SetNumberField(TEXT("Bits"), Record.NumBits);
			OutputFields.Add(MakeShared<FJsonValueObject>(JsonField));
		}

		TSharedPtr<FJsonObject> OutputRoot = MakeShared<FJsonObject>();
		OutputRoot->SetArrayField(TEXT("Fields"), OutputFields);

		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		FJsonSerializer::Serialize(OutputRoot.ToSharedRef(), Writer);

		if (!FFileHelper::SaveStringToFile(OutputString, *OutputJsonPath))
		{
			UE_LOGF(LogTemp, Error, "Unable to save json file: %ls", *OutputJsonPath);
		}
		else
		{
			UE_LOGF(LogTemp, Log, "Bit packed and byte ordered fields exported to json: %ls", *OutputJsonPath);
		}
#endif
	}

	uint16 ComputeAncChecksum(uint16 DID, uint16 SDID, uint16 DataCount, const TArray<uint16>& UDWs)
	{
		static constexpr uint16 Mask9Bits = 0x01FFu;
		uint32 Sum = 0;
		Sum += (DID       & Mask9Bits);
		Sum += (SDID      & Mask9Bits);
		Sum += (DataCount & Mask9Bits);
		for (uint16 Word : UDWs)
		{
			Sum += (Word & Mask9Bits);
		}
		const uint16 Checksum9 = static_cast<uint16>(Sum & Mask9Bits);
		const uint16 Bit8      = (Checksum9 >> 8) & 0x1u;
		const uint16 Bit9      = (~Bit8) & 0x1u;
		return static_cast<uint16>(Checksum9 | (Bit9 << 9));
	}
}

namespace UE::RivermaxCore
{
	FTimecode AtcUDW10ToTimecode(const TArray<uint16>& UDWs)
	{
		if (UDWs.Num() < 16)
		{
			return FTimecode();
		}

		// Each even-indexed UDW carries a BCD nibble at bits b4-b7 (MakeUDW10 layout).
		auto GetNibble = [&](int32 Idx) -> int32 { return (UDWs[Idx] >> 4) & 0xF; };

		const int32 FrameUnits = GetNibble(0);
		const int32 FrameTens  = GetNibble(2) & 0x3;
		const bool  bDropFrame = ((UDWs[2] >> 6) & 1) != 0;
		const int32 SecUnits   = GetNibble(4);
		const int32 SecTens    = GetNibble(6) & 0x7;
		const int32 MinUnits   = GetNibble(8);
		const int32 MinTens    = GetNibble(10) & 0x7;
		const int32 HourUnits  = GetNibble(12);
		const int32 HourTens   = GetNibble(14) & 0x3;

		FTimecode Result;
		Result.Frames           = FrameTens * 10 + FrameUnits;
		Result.Seconds          = SecTens   * 10 + SecUnits;
		Result.Minutes          = MinTens   * 10 + MinUnits;
		Result.Hours            = HourTens  * 10 + HourUnits;
		Result.bDropFrameFormat = bDropFrame;
		return Result;
	}
}