// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RivermaxTypes.h"
#include "Misc/Timecode.h"

namespace UE::RivermaxCore
{
	struct FRivermaxOutputOptions;

	namespace Private
	{
		class FBigEndianHeaderPacker;
	}
}

namespace UE::RivermaxCore::Private::Utils
{	
	/** Various constants used for stream initialization */

	static constexpr uint32 FullHDHeight = 1080;
	static constexpr uint32 FullHDWidth = 1920;

	/** SMTPE 2110-10.The Media Clock and RTP Clock rate for streams compliant to this standard shall be 90 kHz. */
	static constexpr double MediaClockSampleRate = 90000.0;
	
	/** Common sleep time used in places where we are waiting for something to complete */
	static constexpr float SleepTimeSeconds = 50 * 1E-6f;

	/**
	 * Converts a timestamp in MediaClock period units to a frame number for a given frame rate
	 * 2110-20 streams uses a standard media clock rate of 90kHz
	 */
	uint32 TimestampToFrameNumber(uint32 Timestamp, const FFrameRate& FrameRate);

	/** Returns a mediaclock timestamp, for rtp, based on a clock time */
	uint32 GetTimestampFromTime(uint64 InTimeNanosec, double InMediaClockRate);

	/** Convert PTP time to timecode. */
	FTimecode GetTimecodeFromTime(uint64 InTimeNanosec, double InMediaClockRate, const FFrameRate& FrameRate);
	
	/**
	* Creates a Data Identification Word (DID, SDID). Both DID and SDID need to comply with ST291-1 standard:
	* 
	* bits b7 (MSB) through b0 (LSB) shall define the 8-bit SDID word (00h through FFh)
	* bit b8 shall be even parity for bits b7 through b0
	* bit b9 = NOT b8.
	*/
	inline uint16 MakeDataIdentificationWord(uint8 byte8)
	{
		const uint16 Data = byte8;
		const uint16 ParityEven = (FMath::CountBits(Data) & 1) ? 1u : 0u;
		const uint16 bit9 = (~ParityEven) & 1u;
		return static_cast<uint16>(Data | (ParityEven << 8) | (bit9 << 9));
	}

	/**
	* Packs the timecode into user data words (UDW) according to ST 12-2 and returns an array of UDWs already 
	* ready to be put into RTP packet.
	*/
	TArray<uint16> TimecodeToAtcUDW10(const FTimecode& Tc, const FFrameRate& Rate);

	/**
	 * Computes the 10-bit RFC8331 ANC checksum over DID, SDID, DataCount, and UDWs. Only the low 9 bits of each word are summed; bit 9 of the result is NOT b8.
	 * Used by both output (packing) and input (verification).
	 */
	uint16 ComputeAncChecksum(uint16 DID, uint16 SDID, uint16 DataCount, const TArray<uint16>& UDWs);
	
	/** 
	* A helper function that reads a json file with a packet information and its bitfields and outputs json with alignment in platform endianness.
	* This function is only usable for debugging purposes and works only when RIVERMAX_PACKET_DEBUG is set.
	*
	* Sample Input:
	* {
	*	  "Fields": [
	*		{ "Field": "V",  "Bits":  2 },
	*		{ "Field": "P",  "Bits":  1 },
	*		{ "Field": "X",  "Bits":  1 },
	*		{ "Field": "CC", "Bits":  4 },
	*		{ "Field": "M",  "Bits":  1 },
	*		{ "Field": "PT", "Bits":  7 }
	*	]
	* }
	* 
	* Sample Output:
	* {
	*	 "Fields": [
	*		{"Field": "CC", "Bits": 4},
	*		{"Field": "X",  "Bits": 1},
	*		{"Field": "P",  "Bits": 1},
	*		{"Field": "V",  "Bits": 2},
	*		{"Field": "PT", "Bits": 7},
	*		{"Field": "M",  "Bits": 1}
	*  ]
	* }
	* 
	*/
	void GenerateRtpHeaderWithFields(const FString& JsonFilePath, const FString& OutputJsonPath);
}