// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxInAncStream.h"

#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RivermaxLog.h"
#include "RivermaxUtils.h"
#include "RTPHeader.h"

namespace UE::RivermaxCore::Private
{
	FRivermaxInputAncStream::FRivermaxInputAncStream()
	{
		RivermaxStreamType = ERivermaxStreamType::ST2110_40;
	}

	FRivermaxInputAncStream::~FRivermaxInputAncStream()
	{
		Uninitialize();
	}

	FString FRivermaxInputAncStream::GetThreadName() const
	{
		return TEXT("Rivermax ANC InputStream Thread");
	}

	void FRivermaxInputAncStream::OnStreamCreated()
	{
		BufferConfiguration.DataStride = CachedAPI->rmx_input_get_stride_size(&StreamParameters, BufferConfiguration.DataBlockID);
	}

	size_t FRivermaxInputAncStream::GetMaxCompletionChunkSize() const
	{
		return 100;
	}

	void FRivermaxInputAncStream::LogStreamDescriptionOnCreation() const
	{
		UE_LOG(LogRivermax, Display, TEXT("ANC Input stream started listening to %s:%d on interface %s"),
			*Options.StreamAddress, Options.Port, *Options.InterfaceAddress);
	}

	void FRivermaxInputAncStream::PrintStats()
	{
		UE_LOG(LogRivermax, Verbose,
			TEXT("ANC Input stream %d (%s:%u): Packets=%llu, AncOK=%llu, Bytes=%llu, ChecksumErr=%llu, ParseErr=%llu, InvalidHdr=%llu, EmptyCompl=%llu"),
			StreamId,
			*Options.StreamAddress,
			Options.Port,
			StreamStats.PacketsReceived,
			StreamStats.AncPacketsReceived,
			StreamStats.BytesReceived,
			StreamStats.ChecksumErrors,
			StreamStats.ParsingErrors,
			StreamStats.InvalidHeaderCount,
			StreamStats.EmptyCompletionCount);
	}

	void FRivermaxInputAncStream::ParseChunks(const rmx_input_completion* Completion)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::AncParseChunks);
		const size_t PacketCount = rmx_input_get_completion_chunk_size(Completion);

		for (uint64 StrideIndex = 0; StrideIndex < PacketCount; ++StrideIndex)
		{
			const uint8* DataPtr = reinterpret_cast<const uint8*>(rmx_input_get_completion_ptr(Completion, BufferConfiguration.DataBlockID));
			DataPtr += StrideIndex * BufferConfiguration.DataStride;

			const rmx_input_packet_info* PacketInfo = CachedAPI->rmx_input_get_packet_info(&ChunkHandle, StrideIndex);
			const size_t DataSize = rmx_input_get_packet_size(PacketInfo, BufferConfiguration.DataBlockID);

			if (DataSize > 0)
			{
				++StreamStats.PacketsReceived;
				StreamStats.BytesReceived += DataSize;
				ParseAncPacket(DataPtr, DataSize);
			}
			else
			{
				++StreamStats.EmptyCompletionCount;
			}
		}
	}

	void FRivermaxInputAncStream::ParseAncPacket(const uint8* DataPtr, size_t DataSize)
	{
		// Without HDS, Rivermax delivers the full raw Ethernet frame in a single sub-block.
		// Parse the network headers dynamically to handle VLAN tagging and IPv4 options.
		// Minimum: Eth(14) + IPv4(20) + UDP(8) = 42 bytes.

		// Need at least 14 bytes for Ethernet header
		if (DataSize < 14)
		{
			++StreamStats.InvalidHeaderCount;
			return;
		}

		const uint8* HeaderPtr = DataPtr;

		// Skip dst+src MACs (12 bytes), read EtherType
		HeaderPtr += 12;
		uint16 EtherType = (uint16(HeaderPtr[0]) << 8) | HeaderPtr[1];
		HeaderPtr += 2;

		// Handle 802.1Q / QinQ VLAN tags (EtherType 0x8100 or 0x88A8)
		while (EtherType == 0x8100 || EtherType == 0x88A8)
		{
			if ((size_t)(HeaderPtr + 4 - DataPtr) > DataSize)
			{
				++StreamStats.InvalidHeaderCount;
				return;
			}
			HeaderPtr += 2;  // skip VLAN TCI
			EtherType = (uint16(HeaderPtr[0]) << 8) | HeaderPtr[1];
			HeaderPtr += 2;
		}

		// Determine IP header size from IHL (IPv4) or fixed 40 bytes (IPv6)
		size_t IPHeaderSize;
		if (EtherType == 0x0800)  // IPv4
		{
			if ((size_t)(HeaderPtr + 1 - DataPtr) > DataSize)
			{
				++StreamStats.InvalidHeaderCount;
				return;
			}
			IPHeaderSize = size_t(HeaderPtr[0] & 0x0F) * 4;
			if (IPHeaderSize < 20)
			{
				++StreamStats.InvalidHeaderCount;
				return;
			}
		}
		else if (EtherType == 0x86DD)  // IPv6
		{
			IPHeaderSize = 40;
		}
		else
		{
			++StreamStats.InvalidHeaderCount;
			return;
		}

		const size_t NetworkHeaderSize = size_t(HeaderPtr - DataPtr) + IPHeaderSize + 8;  // IP + UDP(8)
		if (DataSize <= NetworkHeaderSize)
		{
			++StreamStats.InvalidHeaderCount;
			return;
		}
		DataPtr += NetworkHeaderSize;
		DataSize -= NetworkHeaderSize;

		FBigEndianHeaderUnpacker Unpacker(DataPtr, (int32)DataSize);

		// RTP header field order matches FillChunk:
		//    V(2) P(1) X(1) CC(4) | M(1) PT(7) | SEQ(16) | Timestamp(32) | SSRC(32) | ExtSeq(16)
		// or 14 bytes total
		constexpr int32 RTPHeaderBits = 112;
		if (!Unpacker.CanRead(RTPHeaderBits))
		{
			++StreamStats.InvalidHeaderCount;
			return;
		}

		// V
		const uint32 Version = Unpacker.GetNextField(2);

		// P+X+CC+M+PT+SEQ
		Unpacker.SkipBits(1 + 1 + 4 + 1 + 7 + 16);

		// Timestamp
		const uint32 Timestamp = Unpacker.GetNextField(32);

		// SSRC+ExtSeq
		Unpacker.SkipBits(32 + 16);

		if (Version != 2)
		{
			++StreamStats.InvalidHeaderCount;
			return;
		}

		const uint32 FrameNumber = UE::RivermaxCore::Private::Utils::TimestampToFrameNumber(Timestamp, Options.FrameRate);


		// ANC payload header — field order matches FillChunk:
		//    Length(16) | ANC_Count(8) | F(2) | Reserved(22)
		// or 6 bytes total
		constexpr int32 ANCPayloadHeaderBits = 48;
		if (!Unpacker.CanRead(ANCPayloadHeaderBits))
		{
			++StreamStats.ParsingErrors;
			Listener->OnAncFrameReceptionError();
			return;
		}

		// Length
		Unpacker.SkipBits(16);

		// ANC_Count
		const uint32 AncCount = Unpacker.GetNextField(8);

		// F+Reserved
		Unpacker.SkipBits(2 + 22);

		if (AncCount == 0)
		{
			return;
		}

		// ANC data packets per RFC8331 Section 8.3 / FillChunk:
		//    C(1)|LineNumber(11)|HorizOffset(12)|S(1)|StreamNum(7)|DID(10)|SDID(10)|DataCount(10)|UDWs(Nx10)|Checksum(10)|WordAlign
		constexpr int32 ANCDataHeaderBits = 62;
		constexpr int32 BitsPerUDW        = 10;
		constexpr int32 BitsPerChecksum   = 10;

		for (uint32 AncIndex = 0; AncIndex < AncCount; ++AncIndex)
		{
			if (!Unpacker.CanRead(ANCDataHeaderBits))
			{
				++StreamStats.ParsingErrors;
				Listener->OnAncFrameReceptionError();
				break;
			}

			Unpacker.SkipBits(1); // C
			const uint16 LineNumber  = (uint16)Unpacker.GetNextField(11);
			const uint16 HorizOffset = (uint16)Unpacker.GetNextField(12);
			Unpacker.SkipBits(1 + 7); // S+StreamNum
			const uint16 DID   = (uint16)Unpacker.GetNextField(10);
			const uint16 SDID  = (uint16)Unpacker.GetNextField(10);
			const uint16 RawDC = (uint16)Unpacker.GetNextField(10);
			const uint8  UDWCount = DecodeDataCount(RawDC);

			if (!Unpacker.CanRead(UDWCount * BitsPerUDW + BitsPerChecksum))
			{
				++StreamStats.ParsingErrors;
				Listener->OnAncFrameReceptionError();
				break;
			}

			TArray<uint16> UDWs;
			UDWs.Reserve(UDWCount);
			for (uint8 iUDW = 0; iUDW < UDWCount; ++iUDW)
			{
				UDWs.Add((uint16)Unpacker.GetNextField(BitsPerUDW));
			}

			const uint16 ReceivedChecksum = (uint16)Unpacker.GetNextField(BitsPerChecksum);

			// Advance to next 32-bit word boundary before processing the next ANC packet (RFC8331 Sec. 8.3).
			const int32 TotalAncBits  = ANCDataHeaderBits + UDWCount * BitsPerUDW + BitsPerChecksum;
			const int32 WordAlignBits = (32 - (TotalAncBits % 32)) % 32;
			Unpacker.SkipBits(WordAlignBits);

			if (!VerifyChecksum(DID, SDID, RawDC, UDWs, ReceivedChecksum))
			{
				++StreamStats.ChecksumErrors;
				Listener->OnAncFrameReceptionError();
				continue;
			}

			++StreamStats.AncPacketsReceived;

			FRivermaxInputAncFrameDescriptor Descriptor;
			Descriptor.Timestamp        = Timestamp;
			Descriptor.FrameNumber      = FrameNumber;
			Descriptor.DID              = DID;
			Descriptor.SDID             = SDID;
			Descriptor.LineNumber       = LineNumber;
			Descriptor.HorizontalOffset = HorizOffset;

			TSharedPtr<IRivermaxAncSample> Sample = Listener->OnAncFrameRequested(Descriptor);
			if (Sample.IsValid())
			{
				Sample->SetAncData(DID, SDID, UDWs);
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Rmax::AncFrameDelivered DID=0x%02X SDID=0x%02X Frame=%u"), DID & 0xFF, SDID & 0xFF, FrameNumber));
				Listener->OnAncFrameReceived(Sample);
			}
		}
	}

	uint8 FRivermaxInputAncStream::DecodeDataCount(uint16 EncodedDataCount)
	{
		return static_cast<uint8>(EncodedDataCount & 0xFFu);
	}

	bool FRivermaxInputAncStream::VerifyChecksum(uint16 DID, uint16 SDID, uint16 DataCount, const TArray<uint16>& UDWs, uint16 ReceivedChecksum)
	{
		return UE::RivermaxCore::Private::Utils::ComputeAncChecksum(DID, SDID, DataCount, UDWs) == ReceivedChecksum;
	}

	void FRivermaxInputAncStream::AllocateBuffers()
	{
		constexpr uint32 CacheLineSize = PLATFORM_CACHE_LINE_SIZE;
		BufferConfiguration.DataMemory->addr = FMemory::Malloc(BufferConfiguration.PayloadSize, CacheLineSize);
		constexpr rmx_mkey_id InvalidKey = static_cast<rmx_mkey_id>(-1L);
		BufferConfiguration.DataMemory->mkey = InvalidKey;
	}

	void FRivermaxInputAncStream::DeallocateBuffers()
	{
		if (BufferConfiguration.DataMemory && BufferConfiguration.DataMemory->addr)
		{
			FMemory::Free(BufferConfiguration.DataMemory->addr);
			BufferConfiguration.DataMemory->addr = nullptr;
		}
	}

	bool FRivermaxInputAncStream::InitializeStreamParameters(FString& OutErrorMessage)
	{
		constexpr size_t BufferCapacity = 1 << 16;
		CachedAPI->rmx_input_set_mem_capacity_in_packets(&StreamParameters, BufferCapacity);

		// ANC does not use DHDS. Rivermax API requires 1 sub-block when HDS is absent.
		constexpr size_t SubBlockCount = 1;
		CachedAPI->rmx_input_set_mem_sub_block_count(&StreamParameters, SubBlockCount);
		CachedAPI->rmx_input_set_entry_size_range(&StreamParameters, BufferConfiguration.DataBlockID, BufferConfiguration.PayloadExpectedSize, BufferConfiguration.PayloadExpectedSize);

		constexpr rmx_input_option           InputOptions = RMX_INPUT_STREAM_CREATE_INFO_PER_PACKET;
		constexpr rmx_input_timestamp_format TimestampFmt = RMX_INPUT_TIMESTAMP_RAW_NANO;
		CachedAPI->rmx_input_enable_stream_option(&StreamParameters, InputOptions);
		CachedAPI->rmx_input_set_timestamp_format(&StreamParameters, TimestampFmt);

		const rmx_status Status = CachedAPI->rmx_input_determine_mem_layout(&StreamParameters);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("ANC: Could not determine memory layout for input stream using IP %s. Status: %d"), *Options.InterfaceAddress, Status);
			return false;
		}

		BufferConfiguration.DataMemory = CachedAPI->rmx_input_get_mem_block_buffer(&StreamParameters, BufferConfiguration.DataBlockID);
		if (!BufferConfiguration.DataMemory || BufferConfiguration.DataMemory->length <= 0)
		{
			OutErrorMessage = FString::Printf(TEXT("ANC: Could not get data memory block for device with IP %s."), *Options.InterfaceAddress);
			return false;
		}

		BufferConfiguration.PayloadSize = BufferConfiguration.DataMemory->length;

		AllocateBuffers();
		return true;
	}
}
