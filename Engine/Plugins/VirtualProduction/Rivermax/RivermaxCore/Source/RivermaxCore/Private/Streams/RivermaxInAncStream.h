// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RivermaxInStream.h"

namespace UE::RivermaxCore::Private
{
	struct FInputAncStreamBufferConfiguration
	{
		size_t PayloadSize = 0;

		const uint16 PayloadExpectedSize = 1500;

		const uint8 DataBlockID = 0;

		rmx_mem_region* DataMemory = nullptr;
		size_t DataStride = 0;
	};

	struct FInputAncStreamStats
	{
		uint64 PacketsReceived = 0;
		uint64 AncPacketsReceived = 0;
		uint64 BytesReceived = 0;
		uint64 ChecksumErrors = 0;
		uint64 ParsingErrors = 0;
		uint64 InvalidHeaderCount = 0;
		uint64 EmptyCompletionCount = 0;
	};

	/** Rivermax input stream for ST2110-40 ANC\. */
	class FRivermaxInputAncStream : public FRivermaxInStream
	{
	public:
		FRivermaxInputAncStream();
		virtual ~FRivermaxInputAncStream();

	protected:
		//~ Begin FRivermaxInStream interface
		virtual FString GetThreadName() const override;
		virtual bool InitializeStreamParameters(FString& OutErrorMessage) override;
		virtual void OnStreamCreated() override;
		virtual size_t GetMaxCompletionChunkSize() const override;
		virtual void DeallocateBuffers() override;
		virtual void ParseChunks(const rmx_input_completion* Completion) override;
		virtual void PrintStats() override;
		virtual void LogStreamDescriptionOnCreation() const override;
		//~ End FRivermaxInStream interface

	private:
		/**
		* Parses a single RFC8331 ANC RTP packet.
		*/
		void ParseAncPacket(const uint8* DataPtr, size_t DataSize);

		/** Returns the lower 8 bits of a parity-encoded 10-bit Data_Count word (the actual UDW count). */
		static uint8 DecodeDataCount(uint16 EncodedDataCount);

		/**
		 * Verifies the 10 bit RFC8331 checksum against DID, SDID, DataCount, and UDWs.
		 */
		static bool VerifyChecksum(uint16 DID, uint16 SDID, uint16 DataCount, const TArray<uint16>& UDWs, uint16 ReceivedChecksum);

		void AllocateBuffers();

	private:
		FInputAncStreamBufferConfiguration BufferConfiguration;
		FInputAncStreamStats StreamStats;
	};
}
