// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxInStream.h"

#include "Async/Async.h"
#include "HAL/CriticalSection.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "Misc/ByteSwap.h"
#include "RivermaxLog.h"
#include "RivermaxUtils.h"

#if PLATFORM_WINDOWS
#include <WS2tcpip.h>
#elif PLATFORM_LINUX
#include <arpa/inet.h>
#endif

namespace UE::RivermaxCore::Private
{
	/** Shared critical section across all input stream types. SDK init calls are not thread-safe. */
	static FCriticalSection GInputStreamInitCriticalSection;

	FRivermaxInStream::FRivermaxInStream()
	{
	}

	FRivermaxInStream::~FRivermaxInStream()
	{
	}

	bool FRivermaxInStream::Initialize(const FRivermaxInputStreamOptions& InOptions, IRivermaxInputStreamListener& InListener)
	{
		IRivermaxCoreModule& RivermaxModule = FModuleManager::LoadModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore"));
		if (RivermaxModule.GetRivermaxManager()->ValidateLibraryIsLoaded() == false)
		{
			UE_LOG(LogRivermax, Warning, TEXT("Can't create Rivermax Input Stream. Library isn't initialized."));
			return false;
		}

		Options = InOptions;
		Listener = &InListener;
		CachedAPI = RivermaxModule.GetRivermaxManager()->GetApi();
		checkSlow(CachedAPI);

		OnPreInitialize(RivermaxModule);

		InitTaskFuture = Async(EAsyncExecution::TaskGraph, [this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxInStream::InitStream);
			FScopeLock Lock(&GInputStreamInitCriticalSection);

			if (bIsShuttingDown.load(std::memory_order_acquire))
			{
				return;
			}

			CachedAPI->rmx_input_init_stream(&StreamParameters, RMX_INPUT_RAW_PACKET);

			FString ErrorMessage;
			FNetworkSettings NetworkSettings;
			bool bWasSuccessful = InitializeNetworkSettings(NetworkSettings, ErrorMessage);
			if (bWasSuccessful)
			{
				bWasSuccessful = InitializeStreamParameters(ErrorMessage);
				if (bWasSuccessful)
				{
					bWasSuccessful = FinalizeStreamCreation(NetworkSettings, ErrorMessage);
				}
			}

			if (bWasSuccessful)
			{
				bIsActive = true;
				RivermaxThread = FRunnableThread::Create(this, *GetThreadName(), 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
				LogStreamDescriptionOnCreation();
			}
			else
			{
				UE_LOG(LogRivermax, Warning, TEXT("%s"), *ErrorMessage);
				DeallocateBuffers();
			}

			FRivermaxInputInitializationResult Result;
			Result.bHasSucceed = bWasSuccessful;
			Result.bIsGPUDirectSupported = GetGPUDirectSupported();
			Listener->OnInitializationCompleted(Result);
		});

		return true;
	}

	void FRivermaxInStream::Uninitialize()
	{
		bIsShuttingDown = true;

		if (InitTaskFuture.IsReady() == false)
		{
			InitTaskFuture.Wait();
		}

		OnPreThreadStop();

		if (RivermaxThread != nullptr)
		{
			Stop();
			RivermaxThread->Kill(true);
			delete RivermaxThread;
			RivermaxThread = nullptr;
			UE_LOG(LogRivermax, Log, TEXT("Rivermax Input stream has shutdown"));
		}

		OnPostThreadStop();

		DeallocateBuffers();
	}

	void FRivermaxInStream::Process_AnyThread()
	{
		checkSlow(CachedAPI);
		rmx_status Status = CachedAPI->rmx_input_get_next_chunk(&ChunkHandle);
		if (Status == RMX_OK)
		{
			ConsecutiveChunkFailures = 0;
			const rmx_input_completion* Completion = CachedAPI->rmx_input_get_chunk_completion(&ChunkHandle);
			if (Completion && rmx_input_get_completion_chunk_size(Completion) > 0)
			{
				ParseChunks(Completion);
			}
		}
		else
		{
			++ConsecutiveChunkFailures;
			if (ConsecutiveChunkFailures == 1)
			{
				UE_LOG(LogRivermax, Warning, TEXT("Input stream failed to get next chunk. Status: %d"), Status);
			}
			else if (ConsecutiveChunkFailures >= MaxConsecutiveChunkFailures)
			{
				UE_LOG(LogRivermax, Error, TEXT("Input stream has failed to get next chunk for %u consecutive attempts (Status: %d). Stopping."), ConsecutiveChunkFailures, Status);
				bIsActive = false;
				Listener->OnStreamError();
				return;
			}
		}

		FPlatformProcess::SleepNoStats(UE::RivermaxCore::Private::Utils::SleepTimeSeconds);
	}

	bool FRivermaxInStream::Init()
	{
		return true;
	}

	uint32 FRivermaxInStream::Run()
	{
		while (bIsActive)
		{
			Process_AnyThread();
			LogStats();
		}

		if (StreamId)
		{
			checkSlow(CachedAPI);
			rmx_status Status = CachedAPI->rmx_input_detach_flow(StreamId, &FlowAttribute);
			if (Status != RMX_OK)
			{
				UE_LOG(LogRivermax, Warning, TEXT("Failed to detach flow from input stream %d. Status: %d"), StreamId, Status);
			}

			Status = CachedAPI->rmx_input_destroy_stream(StreamId);
			if (Status != RMX_OK)
			{
				UE_LOG(LogRivermax, Warning, TEXT("Failed to destroy input stream %d. Status: %d"), StreamId, Status);
			}
		}

		return 0;
	}

	void FRivermaxInStream::Stop()
	{
		bIsActive = false;
	}

	void FRivermaxInStream::Exit()
	{
	}

	bool FRivermaxInStream::InitializeNetworkSettings(FNetworkSettings& OutSettings, FString& OutErrorMessage)
	{
		FMemory::Memset(&OutSettings.DeviceInterface, 0, sizeof(OutSettings.DeviceInterface));
		if (inet_pton(AF_INET, StringCast<ANSICHAR>(*Options.InterfaceAddress).Get(), &OutSettings.DeviceInterface.sin_addr) != 1)
		{
			OutErrorMessage = FString::Printf(TEXT("Failed to convert device interface '%s' to network address"), *Options.InterfaceAddress);
			return false;
		}
		OutSettings.DeviceInterface.sin_family = AF_INET;

		FMemory::Memset(&OutSettings.DestinationAddress, 0, sizeof(OutSettings.DestinationAddress));
		if (inet_pton(AF_INET, StringCast<ANSICHAR>(*Options.StreamAddress).Get(), &OutSettings.DestinationAddress.sin_addr) != 1)
		{
			OutErrorMessage = FString::Printf(TEXT("Failed to convert stream address '%s' to network address"), *Options.StreamAddress);
			return false;
		}
		OutSettings.DestinationAddress.sin_family = AF_INET;
		OutSettings.DestinationAddress.sin_port = ByteSwap((uint16)Options.Port);

		OutSettings.DeviceAddress.family = AF_INET;
		OutSettings.DeviceAddress.addr.ipv4 = OutSettings.DeviceInterface.sin_addr;

		const rmx_status Status = CachedAPI->rmx_retrieve_device_iface(&OutSettings.RMXDeviceInterface, &OutSettings.DeviceAddress);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("Could not retrieve Rivermax interface for IP '%s'"), *Options.InterfaceAddress);
			return false;
		}

		CachedAPI->rmx_input_set_stream_nic_address(&StreamParameters, reinterpret_cast<sockaddr*>(&OutSettings.DeviceInterface));
		return true;
	}

	bool FRivermaxInStream::FinalizeStreamCreation(const FNetworkSettings& NetworkSettings, FString& OutErrorMessage)
	{
		rmx_status Status = CachedAPI->rmx_input_create_stream(&StreamParameters, &StreamId);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("Could not create stream. Status: %d"), Status);
			return false;
		}

		OnStreamCreated();

		const size_t MaxChunkSize = GetMaxCompletionChunkSize();
		Status = CachedAPI->rmx_input_set_completion_moderation(StreamId, 0, MaxChunkSize, 0);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("Could not setup completion moderation. Status: %d"), Status);
			const rmx_status DestroyStatus = CachedAPI->rmx_input_destroy_stream(StreamId);
			if (DestroyStatus != RMX_OK)
			{
				UE_LOG(LogRivermax, Warning, TEXT("Failed to destroy input stream %d after completion moderation failure. Status: %d"), StreamId, DestroyStatus);
			}
			return false;
		}

		CachedAPI->rmx_input_init_chunk_handle(&ChunkHandle, StreamId);
		CachedAPI->rmx_input_init_flow(&FlowAttribute);
		CachedAPI->rmx_input_set_flow_local_addr(&FlowAttribute, reinterpret_cast<const sockaddr*>(&NetworkSettings.DestinationAddress));

		sockaddr_in SourceAddr;
		FMemory::Memset(&SourceAddr, 0, sizeof(SourceAddr));
		SourceAddr.sin_family = AF_INET;
		CachedAPI->rmx_input_set_flow_remote_addr(&FlowAttribute, reinterpret_cast<const sockaddr*>(&SourceAddr));

		CachedAPI->rmx_input_set_flow_tag(&FlowAttribute, FlowTag);

		ConfigureFlowParameters(FlowAttribute);

		Status = CachedAPI->rmx_input_attach_flow(StreamId, &FlowAttribute);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("Could not attach flow to stream. Status: %d"), Status);

			const rmx_status DestroyStatus = CachedAPI->rmx_input_destroy_stream(StreamId);
			if (DestroyStatus != RMX_OK)
			{
				UE_LOG(LogRivermax, Warning, TEXT("Failed to destroy input stream %d after flow attach failure. Status: %d"), StreamId, DestroyStatus);
			}
			return false;
		}

		return true;
	}

	void FRivermaxInStream::LogStats()
	{
		static constexpr double LoggingInterval = 1.0;
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - LastLoggingTimestamp >= LoggingInterval)
		{
			LastLoggingTimestamp = CurrentTime;
			PrintStats();
		}
	}
}
