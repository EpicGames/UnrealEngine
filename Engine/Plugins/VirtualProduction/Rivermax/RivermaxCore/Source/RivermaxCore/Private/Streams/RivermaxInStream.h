// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxInputStream.h"

#include <atomic>
#include "Async/Future.h"
#include "HAL/Runnable.h"
#include "RivermaxWrapper.h"
#include "RivermaxTypes.h"

class IRivermaxCoreModule;

namespace UE::RivermaxCore::Private
{
	using UE::RivermaxCore::IRivermaxInputStream;
	using UE::RivermaxCore::IRivermaxInputStreamListener;
	using UE::RivermaxCore::FRivermaxInputStreamOptions;

	/** Network settings for all input stream types. Video and ANC use identical fields. */
	struct FNetworkSettings
	{
		sockaddr_in DeviceInterface;
		rmx_device_iface RMXDeviceInterface;
		rmx_ip_addr DeviceAddress;
		sockaddr_in DestinationAddress;
	};

	/**
	 * Base class for all Rivermax input streams.
	 * Holds shared infrastructure: thread lifecycle, async init, network setup, reception loop, stream teardown.
	 * Subclasses implement stream-specific behavior via pure virtual hooks.
	 */
	class FRivermaxInStream : public IRivermaxInputStream, public FRunnable
	{
	public:
		FRivermaxInStream();
		virtual ~FRivermaxInStream();

	public:
		//~ Begin IRivermaxInputStream interface
		virtual bool Initialize(const FRivermaxInputStreamOptions& InOptions, IRivermaxInputStreamListener& InListener) override;
		virtual void Uninitialize() override;
		//~ End IRivermaxInputStream interface

		void Process_AnyThread();

		//~ Begin FRunnable interface
		virtual bool Init() override;
		virtual uint32 Run() override;
		virtual void Stop() override;
		virtual void Exit() override;
		//~ End FRunnable interface

	protected:
		/** Called before the async init task. Override for subclass-specific synchronous setup (e.g. cvar reads, HDS enable). */
		virtual void OnPreInitialize(IRivermaxCoreModule& RivermaxModule) {}

		/** Called after init-task wait, before the reception thread is stopped. Override for operations that must precede thread teardown (e.g. CUDA stream sync). */
		virtual void OnPreThreadStop() {}

		/** Called after the reception thread is stopped, before DeallocateBuffers. Override for post-stop cleanup (e.g. disable dynamic header support). */
		virtual void OnPostThreadStop() {}

		/** Name used for the reception thread. */
		virtual FString GetThreadName() const = 0;

		/** Whether this stream instance supports GPUDirect. Queried after initialization completes. */
		virtual bool GetGPUDirectSupported() const { return false; }

		/** Resolves interface/stream IP addresses and configures the NIC address on StreamParameters. */
		bool InitializeNetworkSettings(FNetworkSettings& OutSettings, FString& OutErrorMessage);

		/** Configures sub-block count, memory layout, and triggers buffer allocation. */
		virtual bool InitializeStreamParameters(FString& OutErrorMessage) = 0;

		/** Creates the Rivermax stream, sets up completion moderation, chunk handle, and attaches flow. */
		bool FinalizeStreamCreation(const FNetworkSettings& NetworkSettings, FString& OutErrorMessage);

		/** Called immediately after rmx_input_create_stream succeeds. Override to retrieve strides from StreamParameters into the subclass buffer config. */
		virtual void OnStreamCreated() = 0;

		/** Returns the max chunk size passed to rmx_input_set_completion_moderation. */
		virtual size_t GetMaxCompletionChunkSize() const = 0;

		/** Called before rmx_input_attach_flow. Override to set stream-specific flow parameters. */
		virtual void ConfigureFlowParameters(rmx_input_flow& InFlow) {}

		/** Frees all allocated buffers. Called from Uninitialize after thread teardown. */
		virtual void DeallocateBuffers() = 0;

		/** Processes all packets in the completion chunk. Called each reception loop iteration. */
		virtual void ParseChunks(const rmx_input_completion* Completion) = 0;

		/** Logs stream-specific reception stats. Called every second by LogStats. */
		virtual void PrintStats() = 0;

		/** Logs a human-readable description of the stream after successful creation. */
		virtual void LogStreamDescriptionOnCreation() const = 0;

	private:
		void LogStats();

	protected:
		/** Options used for this stream, such has resolution, frame rate etc... */
		FRivermaxInputStreamOptions Options;

		/** Listener to be notified of stream events */
		IRivermaxInputStreamListener* Listener = nullptr;

		/** Stream id given back by rivermax */
		rmx_stream_id StreamId = 0;

		/** Holding stream configuration data */
		rmx_input_stream_params StreamParameters;

		/** Flow attributes used at stream creation. Used to compare incoming data to our flow and detect mismatches */
		rmx_input_flow FlowAttribute;

		/** Flow identification. Used to verify if packets belong to expected stream */
		uint32 FlowTag = 0;

		/** Type of stream being received */
		ERivermaxStreamType RivermaxStreamType = ERivermaxStreamType::ST2110_20;

		/** Chunk handle shared between Process_AnyThread (base) and ParseChunks (subclass). */
		rmx_input_chunk_handle ChunkHandle;

		/** Thread on which reception is handled */
		FRunnableThread* RivermaxThread = nullptr;

		/** Whether reception thread should keep running */
		std::atomic<bool> bIsActive = false;

		/** Whether stream is in the process of shutting down */
		std::atomic<bool> bIsShuttingDown = false;

		/** Future holding the async initialization task */
		TFuture<void> InitTaskFuture;

		/** Last time stats were logged */
		double LastLoggingTimestamp = 0.0;

		/** Cached pointer to the Rivermax API function list */
		const RIVERMAX_API_FUNCTION_LIST* CachedAPI = nullptr;

		/** Number of consecutive rmx_input_get_next_chunk failures. Resets on any success. */
		uint32 ConsecutiveChunkFailures = 0;

		/** After this many consecutive chunk failures the stream calls OnStreamError and stops. */
		static constexpr uint32 MaxConsecutiveChunkFailures = 1000;
	};
}
