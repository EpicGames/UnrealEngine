// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxInputVideoStream.h"

#include "Async/Async.h"
#if PLATFORM_WINDOWS
#include "CudaModule.h"
#include "ID3D12DynamicRHI.h"
#endif
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "RenderGraphUtils.h"
#include "RivermaxLog.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxTracingUtils.h"
#include "RivermaxUtils.h"
#include "RTPHeader.h"

namespace UE::RivermaxCore::Private
{
#if PLATFORM_WINDOWS
	struct FRivermaxScopedCudaContext
	{
		explicit FRivermaxScopedCudaContext(CUcontext InCtx) : Ctx(InCtx)
		{
			if (Ctx)
			{
				const CUresult Res = FCUDAModule::CUDA().cuCtxPushCurrent(Ctx);
				bPushed = (Res == CUDA_SUCCESS);
#if DO_CHECK
				if (!bPushed)
				{
					UE_LOGF(LogRivermax, Warning, "cuCtxPushCurrent failed (err=%d)", (int32)Res);
				}
#endif
			}
		}

		~FRivermaxScopedCudaContext()
		{
			if (bPushed)
			{
				CUcontext Popped = nullptr;
				const CUresult Res = FCUDAModule::CUDA().cuCtxPopCurrent(&Popped);
#if DO_CHECK
				if (Res != CUDA_SUCCESS)
				{
					UE_LOGF(LogRivermax, Warning, "cuCtxPopCurrent failed (err=%d)", (int32)Res);
				}
				else if (Popped != Ctx)
				{
					UE_LOGF(LogRivermax, Warning, "cuCtxPopCurrent popped unexpected context");
				}
#endif
			}
		}

	private:
		CUcontext Ctx = nullptr;
		bool bPushed = false;
	};
#endif // PLATFORM_WINDOWS

	static TAutoConsoleVariable<float> CVarWaitForCompletionTimeout(
		TEXT("Rivermax.Input.WaitCompletionTimeout"),
		0.25,
		TEXT("Maximum time to wait, in seconds, when waiting for a memory copy operation to complete on the gpu."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarExpectedPayloadSize(
		TEXT("Rivermax.Input.ExpectedPayloadSize"),
		1500,
		TEXT("Expected payload size used to initialize rivermax stream."),
		ECVF_Default);

	static bool GEnableLargeFirstPacketIntervalLogging = false;
	FAutoConsoleVariableRef CVarRivermaxEnableLargeFirstPacketIntervalLogging(
		TEXT("Rivermax.Input.EnableLargeFistPacketIntervalLogging")
		, UE::RivermaxCore::Private::GEnableLargeFirstPacketIntervalLogging
		, TEXT("Enables detection and logging of large delta times between frame boundary and first packet reception.")
		, ECVF_Default);

	static bool GClearFirstPacketIntervalStats = false;
	FAutoConsoleVariableRef CVarRivermaxClearFirstPacketIntervalStats(
		TEXT("Rivermax.Input.ClearFistPacketIntervalStats")
		, UE::RivermaxCore::Private::GClearFirstPacketIntervalStats
		, TEXT("Clears stats related to first packet interval detection.")
		, ECVF_Default);

	static int32 GLargeFirstPacketIntervalThresholdMicroSec = 2000;
	FAutoConsoleVariableRef CVarRivermaxLargeFirstPacketIntervalThreshold(
		TEXT("Rivermax.Input.LargeFistPacketIntervalThreshold")
		, UE::RivermaxCore::Private::GLargeFirstPacketIntervalThresholdMicroSec
		, TEXT("Microseconds required to consider a first packet interval to be large and be logged.")
		, ECVF_Default);


	void FFrameDescriptionTrackingData::ResetSingleFrameTracking()
	{
		PayloadSizeReceived.Empty();
	}

	void FFrameDescriptionTrackingData::EvaluateNewRTP(const FRTPHeader& NewHeader)
	{
		UpdateResolutionDetection(NewHeader);
		UpdatePayloadSizeTracking(NewHeader);

		if (NewHeader.bIsMarkerBit)
		{
			ResetSingleFrameTracking();
		}
	}

	void FFrameDescriptionTrackingData::UpdateResolutionDetection(const FRTPHeader& NewHeader)
	{
		if (NewHeader.bIsMarkerBit)
		{
			const FVideoFormatInfo ExpectedFormatInfo = FStandardVideoFormat::GetVideoFormatInfo(ExpectedSamplingType);
			const uint16 LastPayloadSize = NewHeader.GetLastPayloadSize();
			if (LastPayloadSize % ExpectedFormatInfo.PixelGroupSize == 0)
			{
				DetectedResolution.X = NewHeader.GetLastRowOffset() + (LastPayloadSize / ExpectedFormatInfo.PixelGroupSize * ExpectedFormatInfo.PixelGroupCoverage);
				DetectedResolution.Y = NewHeader.GetLastRowNumber() + 1;

				bHasLoggedSamplingWarning = false;
			}
			else if (!bHasLoggedSamplingWarning)
			{
				bHasLoggedSamplingWarning = true;
				UE_LOGF(LogRivermax, Warning, "Detected incoming signal with unexpected sampling type.");
			}
		}
	}

	void FFrameDescriptionTrackingData::UpdatePayloadSizeTracking(const FRTPHeader& NewHeader)
	{
		if (NewHeader.SRD1.Length > 0)
		{
			uint32 PayloadSize = NewHeader.SRD1.Length;
			if (NewHeader.SRD1.bHasContinuation)
			{
				PayloadSize += NewHeader.SRD2.Length;
			}

			if (PayloadSizeReceived.IsEmpty())
			{
				CommonPayloadSize = PayloadSize;
			}
			else if (!NewHeader.bIsMarkerBit && PayloadSize != CommonPayloadSize)
			{
				CommonPayloadSize = -1;
			}

			PayloadSizeReceived.FindOrAdd(PayloadSize) += 1;
		}
	}

	bool FFrameDescriptionTrackingData::HasDetectedValidResolution() const
	{
		return DetectedResolution.X > 0 && DetectedResolution.Y > 0;
	}

	FRivermaxInputVideoStream::FRivermaxInputVideoStream()
	{
		RivermaxStreamType = ERivermaxStreamType::ST2110_20;
	}

	FRivermaxInputVideoStream::~FRivermaxInputVideoStream()
	{
		Uninitialize();
	}

	void FRivermaxInputVideoStream::OnPreInitialize(IRivermaxCoreModule& RivermaxModule)
	{
		FormatInfo = FStandardVideoFormat::GetVideoFormatInfo(Options.PixelFormat);
		ExpectedPayloadSize = CVarExpectedPayloadSize.GetValueOnGameThread();
		bIsDynamicHeaderEnabled = RivermaxModule.GetRivermaxManager()->EnableDynamicHeaderSupport(Options.InterfaceAddress);

		if (Options.bEnforceVideoFormat)
		{
			StreamResolution = Options.EnforcedResolution;
		}

		FrameDescriptionTracking.ExpectedSamplingType = Options.PixelFormat;
	}

	void FRivermaxInputVideoStream::OnPreThreadStop()
	{
#if PLATFORM_WINDOWS
		if (bIsUsingGPUDirect && GPUStream)
		{
			FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
			FRivermaxScopedCudaContext CudaScopedContext(CudaModule.GetCudaContextForDevice(GPUDeviceIndex));
			const CUresult Status = CudaModule.DriverAPI()->cuStreamSynchronize(reinterpret_cast<CUstream>(GPUStream));
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "CUDA Failed to synchronize stream. Status: %d", Status);
			}
		}
#endif // PLATFORM_WINDOWS
	}

	void FRivermaxInputVideoStream::OnPostThreadStop()
	{
		if (bIsDynamicHeaderEnabled)
		{
			if (IRivermaxCoreModule* RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>(TEXT("RivermaxCore")))
			{
				RivermaxModule->GetRivermaxManager()->DisableDynamicHeaderSupport(Options.InterfaceAddress);
			}
			bIsDynamicHeaderEnabled = false;
		}
	}

	FString FRivermaxInputVideoStream::GetThreadName() const
	{
		return TEXT("Rivermax InputStream Thread");
	}

	bool FRivermaxInputVideoStream::GetGPUDirectSupported() const
	{
		return bIsUsingGPUDirect;
	}

	void FRivermaxInputVideoStream::OnStreamCreated()
	{
		BufferConfiguration.DataStride = CachedAPI->rmx_input_get_stride_size(&StreamParameters, BufferConfiguration.DataBlockID);
		BufferConfiguration.HeaderStride = CachedAPI->rmx_input_get_stride_size(&StreamParameters, BufferConfiguration.HeaderBlockID);
	}

	size_t FRivermaxInputVideoStream::GetMaxCompletionChunkSize() const
	{
		return 5000;
	}

	void FRivermaxInputVideoStream::LogStreamDescriptionOnCreation() const
	{
		UE_LOGF(LogRivermax, Display, "Input stream started listening to stream %ls:%d on interface %ls%ls"
			, *Options.StreamAddress
			, Options.Port
			, *Options.InterfaceAddress
			, bIsUsingGPUDirect ? TEXT(" using GPUDirect") : TEXT(""));
	}

	void FRivermaxInputVideoStream::PrintStats()
	{
		UE_LOGF(LogRivermax, Verbose, "Stream %d (%ls:%u) stats: FrameCount: %llu, EndOfFrame: %llu, Chunks: %llu, Bytes: %llu, PacketLossInFrame: %llu, TotalPacketLoss: %llu, BiggerFrames: %llu, InvalidFrames: %llu, InvalidHeader: %llu, EmptyCompletion: %llu"
			, StreamId
			, *Options.StreamAddress
			, Options.Port
			, StreamStats.FramesReceived
			, StreamStats.EndOfFrameReceived
			, StreamStats.ChunksReceived
			, StreamStats.BytesReceived
			, StreamStats.FramePacketLossCount
			, StreamStats.TotalPacketLossCount
			, StreamStats.BiggerFramesCount
			, StreamStats.InvalidFramesCount
			, StreamStats.InvalidHeadercount
			, StreamStats.EmptyCompletionCount
		);

		UE_LOGF(LogRivermax, Verbose, "Stream %d (%ls:%u) first packet interval: - Min: %llu. Max: %llu. Avg: %llu."
			, StreamId
			, *Options.StreamAddress
			, Options.Port
			, StreamStats.MinFirstPacketIntervalNS
			, StreamStats.MaxFirstPacketIntervalNS
			, StreamStats.FirstPacketIntervalAccumulatorNS);
	}

	void FRivermaxInputVideoStream::ParseChunks(const rmx_input_completion* Completion)
	{
		const size_t PacketCount = rmx_input_get_completion_chunk_size(Completion);

		for (uint64 StrideIndex = 0; StrideIndex < PacketCount; ++StrideIndex)
		{
			++StreamStats.ChunksReceived;

			const uint8* RawHeaderPtr = reinterpret_cast<const uint8_t*>(rmx_input_get_completion_ptr(Completion, BufferConfiguration.HeaderBlockID));
			const uint8* DataPtr = reinterpret_cast<const uint8_t*>(rmx_input_get_completion_ptr(Completion, BufferConfiguration.DataBlockID));

			RawHeaderPtr += StrideIndex * BufferConfiguration.HeaderStride;
			DataPtr += StrideIndex * BufferConfiguration.DataStride;

			const rmx_input_packet_info* PacketInfo = CachedAPI->rmx_input_get_packet_info(&ChunkHandle, StrideIndex);
			const size_t PacketSize = rmx_input_get_packet_size(PacketInfo, BufferConfiguration.DataBlockID);
			const size_t HeaderSize = rmx_input_get_packet_size(PacketInfo, BufferConfiguration.HeaderBlockID);
			if (PacketSize > 0 && HeaderSize > 0)
			{
				if (FlowTag)
				{
					const uint32 PacketTag = rmx_input_get_packet_flow_tag(PacketInfo);
					if (PacketTag != FlowTag)
					{
						UE_LOGF(LogRivermax, Error, "Received data from unexpected FlowTag '%d'. Expected '%d'.", PacketTag, FlowTag);
						Listener->OnStreamError();
						bIsShuttingDown = true;
						return;
					}
				}

				const FVideoRTPHeader& RawRTPHeaderPtr = reinterpret_cast<const FVideoRTPHeader&>(*GetRTPHeaderPointerVideo(RawHeaderPtr));
				if (RawRTPHeaderPtr.RTPHeader.Version == 2)
				{
					FRTPHeader RTPHeader(RawRTPHeaderPtr);

					if (bIsFirstPacketReceived == false)
					{
						const uint32 FrameNumber = Utils::TimestampToFrameNumber(RTPHeader.Timestamp, Options.FrameRate);
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxInStartingFrameTraceEvents[FrameNumber % 10]);

						IRivermaxCoreModule& RivermaxModule = FModuleManager::LoadModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore"));
						const uint64 CurrentTime = RivermaxModule.GetRivermaxManager()->GetTime();

						bIsFirstPacketReceived = true;

						if (StreamStats.EndOfFrameReceived > 0)
						{
							const uint64 FrameBoundary = GetAlignmentPointFromFrameNumber(GetFrameNumber(CurrentTime, Options.FrameRate), Options.FrameRate);
							const uint64 FirstPacketInterval = CurrentTime - FrameBoundary;
							StreamStats.MinFirstPacketIntervalNS = FMath::Min(StreamStats.MinFirstPacketIntervalNS, FirstPacketInterval);
							StreamStats.MaxFirstPacketIntervalNS = FMath::Max(StreamStats.MaxFirstPacketIntervalNS, FirstPacketInterval);

							constexpr double Alpha = 0.8;
							StreamStats.FirstPacketIntervalAccumulatorNS = Alpha * FirstPacketInterval + ((1.0 - Alpha) * StreamStats.FirstPacketIntervalAccumulatorNS);

							if (GClearFirstPacketIntervalStats)
							{
								GClearFirstPacketIntervalStats = false;
								StreamStats.MinFirstPacketIntervalNS = TNumericLimits<uint64>::Max();
								StreamStats.MaxFirstPacketIntervalNS = TNumericLimits<uint64>::Min();
								StreamStats.FirstPacketIntervalAccumulatorNS = 0;
							}

							if (GEnableLargeFirstPacketIntervalLogging)
							{
								const uint32 IntervalMicroSec = FirstPacketInterval / 1000;
								if (IntervalMicroSec > (uint32)GLargeFirstPacketIntervalThresholdMicroSec)
								{
									UE_LOGF(LogRivermax, Warning, "Large First packet interval: %llu. Min: %llu. Max: %llu. Avg: %llu."
										, FirstPacketInterval
										, StreamStats.MinFirstPacketIntervalNS
										, StreamStats.MaxFirstPacketIntervalNS
										, StreamStats.FirstPacketIntervalAccumulatorNS);
								}
							}
						}
					}

					StreamStats.BytesReceived += PacketSize + HeaderSize;

					UpdateFrameTracking(RTPHeader);

					if (!StreamData.CurrentSamples.Contains(IRivermaxSample::ESampleType::Video) && State != EReceptionState::FrameError)
					{
						TSharedPtr<IRivermaxVideoSample> Sample = GetVideoSampleForReception(RTPHeader);
						StreamData.CurrentSamples.Add(IRivermaxSample::ESampleType::Video, StaticCastSharedPtr<IRivermaxSample>(Sample));
					}

					switch (State)
					{
					case EReceptionState::Receiving:
					{
						FrameReceptionState(RTPHeader, DataPtr, StreamData.CurrentSamples[IRivermaxSample::ESampleType::Video]);
						break;
					}
					case EReceptionState::WaitingForMarker:
					{
						WaitForMarkerState(RTPHeader);
						break;
					}
					case EReceptionState::FrameError:
					{
						FrameErrorState(RTPHeader);
						break;
					}
					default:
					{
						checkNoEntry();
					}
					}
				}
				else
				{
					++StreamStats.InvalidHeadercount;
				}
			}
			else
			{
				++StreamStats.EmptyCompletionCount;
			}
		}
	}

	TSharedPtr<IRivermaxVideoSample> FRivermaxInputVideoStream::GetVideoSampleForReception(const FRTPHeader& RTPHeader)
	{
		using namespace UE::RivermaxCore::Private::Utils;

		FRivermaxInputVideoFrameDescriptor Descriptor;
		Descriptor.Timestamp = RTPHeader.Timestamp;
		Descriptor.FrameNumber = TimestampToFrameNumber(RTPHeader.Timestamp, Options.FrameRate);
		Descriptor.Width = StreamResolution.X;
		Descriptor.Height = StreamResolution.Y;
		Descriptor.PixelFormat = Options.PixelFormat;
		const uint32 PixelCount = StreamResolution.X * StreamResolution.Y;
		const uint32 FrameSize = PixelCount / FormatInfo.PixelGroupCoverage * FormatInfo.PixelGroupSize;
		Descriptor.VideoBufferSize = FrameSize;

		TSharedPtr<IRivermaxVideoSample> OutSample = Listener->OnVideoFrameRequested(Descriptor);

		StreamData.CurrentFrameVideoBuffer = nullptr;
		if (OutSample.IsValid())
		{
			if (bIsUsingGPUDirect)
			{
				FBufferRHIRef RHIBuffer = OutSample->GetGPUBuffer()->GetRHI();
				if (RHIBuffer.IsValid())
				{
					StreamData.CurrentFrameVideoBuffer = GetMappedBuffer(RHIBuffer);
				}
			}
			else
			{
				StreamData.CurrentFrameVideoBuffer = OutSample->GetVideoBufferRawPtr(Descriptor.VideoBufferSize);
			}
		}

		if (StreamData.CurrentFrameVideoBuffer == nullptr)
		{
			UE_LOGF(LogRivermax, Verbose, "Could not get a new frame for incoming frame with timestamp %u and frame number %u", RTPHeader.Timestamp, Descriptor.FrameNumber);
			Listener->OnVideoFrameReceptionError(OutSample);
			State = EReceptionState::FrameError;
			FrameErrorState(RTPHeader);
		}
		else
		{
			StreamData.WritingOffset = 0;
			StreamData.ReceivedSize = 0;
			StreamData.ExpectedSize = Descriptor.VideoBufferSize;
			StreamData.DeviceWritePointerOne = nullptr;
			StreamData.SizeToWriteOne = 0;
			StreamData.DeviceWritePointerTwo = nullptr;
			StreamData.SizeToWriteTwo = 0;
			bIsFirstPacketReceived = false;
		}

		return OutSample;
	}

	void FRivermaxInputVideoStream::AllocateBuffers()
	{
		IRivermaxCoreModule& RivermaxModule = FModuleManager::LoadModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore"));
		if (RivermaxModule.GetRivermaxManager()->IsGPUDirectInputSupported() && Options.bUseGPUDirect)
		{
			bIsUsingGPUDirect = AllocateGPUBuffers();
		}

		constexpr uint32 CacheLineSize = PLATFORM_CACHE_LINE_SIZE;
		if (bIsUsingGPUDirect == false)
		{
			BufferConfiguration.DataMemory->addr = FMemory::Malloc(BufferConfiguration.PayloadSize, CacheLineSize);
		}

		BufferConfiguration.HeaderMemory->addr = FMemory::Malloc(BufferConfiguration.HeaderSize, CacheLineSize);

		constexpr rmx_mkey_id InvalidKey = ((rmx_mkey_id)(-1L));
		BufferConfiguration.DataMemory->mkey = InvalidKey;
		BufferConfiguration.HeaderMemory->mkey = InvalidKey;
	}

	bool FRivermaxInputVideoStream::AllocateGPUBuffers()
	{
#if PLATFORM_WINDOWS
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxInputVideoStream::AllocateGPUBuffers);

		const ERHIInterfaceType RHIType = RHIGetInterfaceType();
		if (RHIType != ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. RHI is %d but only Dx12 is supported at the moment.", int(RHIType));
			return false;
		}

		FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
		GPUDeviceIndex = CudaModule.GetCudaDeviceIndex();
		FRivermaxScopedCudaContext CudaScopedContext(CudaModule.GetCudaContextForDevice(GPUDeviceIndex));

		CUdevice CudaDevice;
		CUresult Status = CudaModule.DriverAPI()->cuDeviceGet(&CudaDevice, GPUDeviceIndex);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to get a Cuda device for GPU %d. Status: %d", GPUDeviceIndex, Status);
			return false;
		}

		CUmemAllocationProp AllocationProperties = {};
		AllocationProperties.type = CU_MEM_ALLOCATION_TYPE_PINNED;
		AllocationProperties.allocFlags.gpuDirectRDMACapable = 1;
		AllocationProperties.allocFlags.usage = 0;
		AllocationProperties.location.type = CU_MEM_LOCATION_TYPE_DEVICE;
		AllocationProperties.location.id = CudaDevice;

		size_t Granularity;
		Status = CudaModule.DriverAPI()->cuMemGetAllocationGranularity(&Granularity, &AllocationProperties, CU_MEM_ALLOC_GRANULARITY_RECOMMENDED);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to get allocation granularity. Status: %d", Status);
			return false;
		}

		const size_t CudaAlignedAllocation = (BufferConfiguration.PayloadSize % Granularity) ? BufferConfiguration.PayloadSize + (Granularity - (BufferConfiguration.PayloadSize % Granularity)) : BufferConfiguration.PayloadSize;

		CUdeviceptr CudaBaseAddress;
		constexpr CUdeviceptr InitialAddress = 0;
		constexpr int32 Flags = 0;
		Status = CudaModule.DriverAPI()->cuMemAddressReserve(&CudaBaseAddress, CudaAlignedAllocation, Granularity, InitialAddress, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to reserve memory for %llu bytes. Status: %d", (unsigned long long)CudaAlignedAllocation, Status);
			return false;
		}

		CUmemGenericAllocationHandle Handle;
		Status = CudaModule.DriverAPI()->cuMemCreate(&Handle, CudaAlignedAllocation, &AllocationProperties, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to create memory on device. Status: %d", Status);
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}
		UE_LOGF(LogRivermax, Verbose, "Allocated %llu cuda memory", (unsigned long long)CudaAlignedAllocation);

		bool bMapped = false;
		constexpr int32 Offset = 0;
		Status = CudaModule.DriverAPI()->cuMemMap(CudaBaseAddress, CudaAlignedAllocation, Offset, Handle, Flags);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to map memory. Status: %d", Status);
		}
		else
		{
			bMapped = true;
		}

		Status = CudaModule.DriverAPI()->cuMemRelease(Handle);
		if (Status != CUDA_SUCCESS)
		{
			// cuMemRelease failure is a CUDA driver-level error. The physical allocation backing Handle
			// cannot be recovered — it will leak. Best-effort: release the VA space and mapping.
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to release handle (physical GPU memory leaked). Status: %d", Status);
			if (bMapped)
			{
				CudaModule.DriverAPI()->cuMemUnmap(CudaBaseAddress, CudaAlignedAllocation);
			}
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}

		if (!bMapped)
		{
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}

		CUmemAccessDesc MemoryAccessDescription = {};
		MemoryAccessDescription.flags = CU_MEM_ACCESS_FLAGS_PROT_READWRITE;
		MemoryAccessDescription.location.type = CU_MEM_LOCATION_TYPE_DEVICE;
		MemoryAccessDescription.location.id = CudaDevice;
		constexpr int32 Count = 1;
		Status = CudaModule.DriverAPI()->cuMemSetAccess(CudaBaseAddress, CudaAlignedAllocation, &MemoryAccessDescription, Count);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to configure memory access. Status: %d", Status);
			CudaModule.DriverAPI()->cuMemUnmap(CudaBaseAddress, CudaAlignedAllocation);
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}

		CUstream CudaStream;
		Status = CudaModule.DriverAPI()->cuStreamCreate(&CudaStream, CU_STREAM_NON_BLOCKING);
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to create its stream. Status: %d", Status);
			CudaModule.DriverAPI()->cuMemUnmap(CudaBaseAddress, CudaAlignedAllocation);
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}

		Status = CudaModule.DriverAPI()->cuCtxSynchronize();
		if (Status != CUDA_SUCCESS)
		{
			UE_LOGF(LogRivermax, Warning, "Can't initialize input to use GPUDirect. Failed to synchronize context. Status: %d", Status);
			CudaModule.DriverAPI()->cuStreamDestroy(CudaStream);
			CudaModule.DriverAPI()->cuMemUnmap(CudaBaseAddress, CudaAlignedAllocation);
			CudaModule.DriverAPI()->cuMemAddressFree(CudaBaseAddress, CudaAlignedAllocation);
			return false;
		}

		// All CUDA initialization succeeded — commit state.
		GPUAllocatedMemorySize = CudaAlignedAllocation;
		GPUAllocatedMemoryBaseAddress = reinterpret_cast<void*>(CudaBaseAddress);
		GPUStream = CudaStream;
		BufferConfiguration.DataMemory->addr = GPUAllocatedMemoryBaseAddress;
		CallbackPayload = MakeShared<FCallbackPayload>();

		return true;
#else
		return false;
#endif // PLATFORM_WINDOWS
	}

	void FRivermaxInputVideoStream::DeallocateBuffers()
	{
#if PLATFORM_WINDOWS
		if (GPUAllocatedMemorySize > 0)
		{
			FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
			FRivermaxScopedCudaContext CudaScopedContext(CudaModule.GetCudaContextForDevice(GPUDeviceIndex));

			const CUdeviceptr BaseAddress = reinterpret_cast<CUdeviceptr>(GPUAllocatedMemoryBaseAddress);
			CUresult Status = CudaModule.DriverAPI()->cuMemUnmap(BaseAddress, GPUAllocatedMemorySize);
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to unmap cuda memory used for input stream. Status: %d", Status);
			}

			Status = CudaModule.DriverAPI()->cuMemAddressFree(BaseAddress, GPUAllocatedMemorySize);
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to free cuda memory used for input stream. Status: %d", Status);
			}
			UE_LOGF(LogRivermax, Verbose, "Deallocated %llu cuda memory at address %p", (unsigned long long)GPUAllocatedMemorySize, GPUAllocatedMemoryBaseAddress);

			GPUAllocatedMemorySize = 0;
			GPUAllocatedMemoryBaseAddress = 0;

			for (const TPair<FRHIBuffer*, FCudaExternalBufferMapping>& Entry : BufferGPUMemoryMap)
			{
				if (Entry.Value.ExternalMemory)
				{
					Status = CudaModule.DriverAPI()->cuDestroyExternalMemory(reinterpret_cast<CUexternalMemory>(Entry.Value.ExternalMemory));
					if (Status != CUDA_SUCCESS)
					{
						UE_LOGF(LogRivermax, Error, "CUDA: Error cleaning up external memory mapping. Error: %d", Status);
					}
				}
			}
			BufferGPUMemoryMap.Empty();

			Status = CudaModule.DriverAPI()->cuStreamDestroy(reinterpret_cast<CUstream>(GPUStream));
			if (Status != CUDA_SUCCESS)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to destroy cuda stream. Status: %d", Status);
			}
			GPUStream = nullptr;
		}
#endif // PLATFORM_WINDOWS

		if (BufferConfiguration.HeaderMemory && BufferConfiguration.HeaderMemory->addr)
		{
			FMemory::Free(BufferConfiguration.HeaderMemory->addr);
			BufferConfiguration.HeaderMemory->addr = nullptr;
		}

		if (bIsUsingGPUDirect == false && BufferConfiguration.DataMemory && BufferConfiguration.DataMemory->addr)
		{
			FMemory::Free(BufferConfiguration.DataMemory->addr);
			BufferConfiguration.DataMemory->addr = nullptr;
		}
	}

	void* FRivermaxInputVideoStream::GetMappedBuffer(const FBufferRHIRef& InBuffer)
	{
#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();
		check(RHIType == ERHIInterfaceType::D3D12);

		if (BufferGPUMemoryMap.Find((InBuffer)) == nullptr)
		{
			int64 BufferMemorySize = 0;
			CUexternalMemory MappedExternalMemory = nullptr;
			HANDLE D3D12BufferHandle = 0;
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC CudaExtMemHandleDesc = {};

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxInput_D3D12CreateSharedHandle);

				ID3D12Resource* NativeD3D12Resource = GetID3D12DynamicRHI()->RHIGetResource(InBuffer);
				BufferMemorySize = GetID3D12DynamicRHI()->RHIGetResourceMemorySize(InBuffer);

				TRefCountPtr<ID3D12Device> OwnerDevice;
				HRESULT QueryResult;
				if ((QueryResult = NativeD3D12Resource->GetDevice(IID_PPV_ARGS(OwnerDevice.GetInitReference()))) != S_OK)
				{
					UE_LOGF(LogRivermax, Error, "Failed to get D3D12 device for captured buffer ressource: %d)", QueryResult);
					return nullptr;
				}

				if ((QueryResult = OwnerDevice->CreateSharedHandle(NativeD3D12Resource, NULL, GENERIC_ALL, NULL, &D3D12BufferHandle)) != S_OK)
				{
					UE_LOGF(LogRivermax, Error, "Failed to create shared handle for captured buffer ressource: %d", QueryResult);
					return nullptr;
				}

				CudaExtMemHandleDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D12_RESOURCE;
				CudaExtMemHandleDesc.handle.win32.name = nullptr;
				CudaExtMemHandleDesc.handle.win32.handle = D3D12BufferHandle;
				CudaExtMemHandleDesc.size = BufferMemorySize;
				CudaExtMemHandleDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;
			}

			FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");

			FRivermaxScopedCudaContext CudaScopedContext(CudaModule.GetCudaContextForDevice(GPUDeviceIndex));

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax_CudaImportMemory);

				const CUresult Result = FCUDAModule::CUDA().cuImportExternalMemory(&MappedExternalMemory, &CudaExtMemHandleDesc);

				if (D3D12BufferHandle)
				{
					CloseHandle(D3D12BufferHandle);
					D3D12BufferHandle = nullptr;
				}

				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogRivermax, Error, "Failed to import shared buffer. Error: %d", Result);
					return nullptr;
				}
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Rmax_MapCudaMemory);

				CUDA_EXTERNAL_MEMORY_BUFFER_DESC BufferDescription = {};
				BufferDescription.offset = 0;
				BufferDescription.size = BufferMemorySize;
				CUdeviceptr NewMemory = 0;
				const CUresult Result = FCUDAModule::CUDA().cuExternalMemoryGetMappedBuffer(&NewMemory, MappedExternalMemory, &BufferDescription);
				if (Result != CUDA_SUCCESS || NewMemory == 0)
				{
					UE_LOGF(LogRivermax, Error, "Failed to get shared buffer mapped memory. Error: %d", Result);
					FCUDAModule::CUDA().cuDestroyExternalMemory(MappedExternalMemory);
					return nullptr;
				}
				FCudaExternalBufferMapping MappingInfo;
				MappingInfo.ExternalMemory = MappedExternalMemory;
				MappingInfo.MappedPtr = reinterpret_cast<void*>(NewMemory);

				BufferGPUMemoryMap.Add(InBuffer, MappingInfo);
			}
		}

		return BufferGPUMemoryMap[InBuffer].MappedPtr;
#else
		return nullptr;
#endif // PLATFORM_WINDOWS
	}

	void FRivermaxInputVideoStream::ProcessSRD(const FRTPHeader& RTPHeader, const uint8* DataPtr)
	{
		if (RTPHeader.SRD1.Length <= 0)
		{
			return;
		}

		uint32 DataOffset = 0;
		uint32 PayloadSize = RTPHeader.SRD1.Length;
		if (RTPHeader.SRD1.bHasContinuation)
		{
			PayloadSize += RTPHeader.SRD2.Length;
		}

		if (bIsUsingGPUDirect)
		{
			if (StreamData.DeviceWritePointerOne == nullptr)
			{
				StreamData.DeviceWritePointerOne = DataPtr;
				StreamData.SizeToWriteOne = PayloadSize;
			}
			else
			{
				if (StreamData.DeviceWritePointerTwo == nullptr && DataPtr < StreamData.DeviceWritePointerOne)
				{
					StreamData.DeviceWritePointerTwo = DataPtr;
					StreamData.SizeToWriteTwo = 0;
				}

				if (StreamData.DeviceWritePointerTwo == nullptr)
				{
					StreamData.SizeToWriteOne += PayloadSize;
				}
				else
				{
					StreamData.SizeToWriteTwo += PayloadSize;
				}
			}
		}
		else
		{
			uint8* WriteBuffer = reinterpret_cast<uint8*>(StreamData.CurrentFrameVideoBuffer);
			FMemory::Memcpy(&WriteBuffer[StreamData.WritingOffset], &DataPtr[DataOffset], RTPHeader.SRD1.Length);
			StreamData.WritingOffset += RTPHeader.SRD1.Length;

			if (RTPHeader.SRD1.bHasContinuation)
			{
				DataOffset += RTPHeader.SRD1.Length;
				FMemory::Memcpy(&WriteBuffer[StreamData.WritingOffset], &DataPtr[DataOffset], RTPHeader.SRD2.Length);
				StreamData.WritingOffset += RTPHeader.SRD2.Length;
			}
		}

		StreamData.ReceivedSize += PayloadSize;
	}

	void FRivermaxInputVideoStream::ProcessLastSRD(const FRTPHeader& RTPHeader, const uint8* DataPtr, TSharedPtr<IRivermaxSample> InSample)
	{
		if (StreamData.ReceivedSize == StreamData.ExpectedSize)
		{
			++StreamStats.FramesReceived;
			uint32 FrameNumber = Utils::TimestampToFrameNumber(RTPHeader.Timestamp, Options.FrameRate);
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxInReceivedFrameTraceEvents[FrameNumber % 10]);

			if (bIsUsingGPUDirect)
			{
#if PLATFORM_WINDOWS
				if (FrameDescriptionTracking.CommonPayloadSize <= 0)
				{
					UE_LOGF(LogRivermax, Warning, "Unsupported variable SRD length detected while GPUDirect is used. Disable GPUDirect and reopen the stream.");
					Listener->OnStreamError();
					bIsShuttingDown.store(true, std::memory_order_release);

					Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
					State = EReceptionState::FrameError;
					FrameErrorState(RTPHeader);
					return;
				}

				FCUDAModule& CudaModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
				FRivermaxScopedCudaContext CudaScopedContext(CudaModule.GetCudaContextForDevice(GPUDeviceIndex));

				const CUdeviceptr DestinationGPUMemory = reinterpret_cast<CUdeviceptr>(StreamData.CurrentFrameVideoBuffer);
				const CUdeviceptr SourceGPUMemoryOne = reinterpret_cast<CUdeviceptr>(StreamData.DeviceWritePointerOne);

				const uint32 NumSRDPartOne = StreamData.SizeToWriteOne / FrameDescriptionTracking.CommonPayloadSize;
				const uint32 NumSRDPartTwo = StreamData.SizeToWriteTwo / FrameDescriptionTracking.CommonPayloadSize;

				CUDA_MEMCPY2D StrideDescription;
				FMemory::Memset(StrideDescription, 0);
				StrideDescription.srcDevice = SourceGPUMemoryOne;
				StrideDescription.dstDevice = DestinationGPUMemory;
				StrideDescription.dstMemoryType = CUmemorytype::CU_MEMORYTYPE_DEVICE;
				StrideDescription.srcMemoryType = CUmemorytype::CU_MEMORYTYPE_DEVICE;
				StrideDescription.srcPitch = ExpectedPayloadSize;
				StrideDescription.dstPitch = FrameDescriptionTracking.CommonPayloadSize;
				StrideDescription.WidthInBytes = FrameDescriptionTracking.CommonPayloadSize;
				StrideDescription.Height = NumSRDPartOne;
				CUresult Result = CudaModule.DriverAPI()->cuMemcpy2DAsync(&StrideDescription, reinterpret_cast<CUstream>(GPUStream));

				CUdeviceptr SourcePtr = SourceGPUMemoryOne + (ExpectedPayloadSize * NumSRDPartOne);

				if (StreamData.DeviceWritePointerTwo != nullptr && StreamData.SizeToWriteTwo > 0)
				{
					StrideDescription.srcDevice = reinterpret_cast<CUdeviceptr>(StreamData.DeviceWritePointerTwo);
					StrideDescription.dstDevice = reinterpret_cast<CUdeviceptr>(StreamData.CurrentFrameVideoBuffer) + StreamData.SizeToWriteOne;
					StrideDescription.Height = NumSRDPartTwo;
					Result = CudaModule.DriverAPI()->cuMemcpy2DAsync(&StrideDescription, reinterpret_cast<CUstream>(GPUStream));

					SourcePtr = StrideDescription.srcDevice + (ExpectedPayloadSize * NumSRDPartTwo);
				}

				const uint32 TotalMemoryCopied = (FrameDescriptionTracking.CommonPayloadSize * (NumSRDPartOne + NumSRDPartTwo));
				if (TotalMemoryCopied < StreamData.ExpectedSize)
				{
					const uint32 LastPacketBytes = RTPHeader.SRD1.bHasContinuation ? (RTPHeader.SRD1.Length + RTPHeader.SRD2.Length) : RTPHeader.SRD1.Length;

					const uint32 SizeLeftToCopy = StreamData.ExpectedSize - TotalMemoryCopied;
					if (SizeLeftToCopy > 0)
					{
						if (ensure(SizeLeftToCopy == LastPacketBytes))
						{
							CUdeviceptr DestinationPtr = DestinationGPUMemory + TotalMemoryCopied;
							Result = CudaModule.DriverAPI()->cuMemcpyDtoDAsync(DestinationPtr, SourcePtr, SizeLeftToCopy, reinterpret_cast<CUstream>(GPUStream));
						}
						else
						{
							UE_LOGF(LogRivermax, Warning, "Unexpected tail copy size: left=%u lastPacket=%u", SizeLeftToCopy, LastPacketBytes);
							Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
							State = EReceptionState::FrameError;
							FrameErrorState(RTPHeader);
							return;
						}
					}
				}

				if (Result != CUDA_SUCCESS)
				{
					UE_LOGF(LogRivermax, Warning, "Failed to copy received buffer to shared memory. Error: %d", Result);
					Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
					State = EReceptionState::FrameError;
					FrameErrorState(RTPHeader);
					return;
				}

				auto CudaCallback = [](void* UserData)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxInputVideoStream::MemcopyCallback);
					if (UserData)
					{
						TUniquePtr<TWeakPtr<FCallbackPayload>> WeakPtrHolder(static_cast<TWeakPtr<FCallbackPayload>*>(UserData));
						if (TSharedPtr<FCallbackPayload> Payload = WeakPtrHolder->Pin())
						{
							Payload->bIsWaitingForPendingCopy.store(false, std::memory_order_release);
						}
					}
				};

				CallbackPayload->bIsWaitingForPendingCopy.store(true, std::memory_order_release);
				TWeakPtr<FCallbackPayload>* WeakPayloadPtr = new TWeakPtr<FCallbackPayload>(CallbackPayload);
				Result = CudaModule.DriverAPI()->cuLaunchHostFunc(reinterpret_cast<CUstream>(GPUStream), CudaCallback, WeakPayloadPtr);
				if (Result != CUDA_SUCCESS)
				{
					delete WeakPayloadPtr;

					UE_LOGF(LogRivermax, Error, "Failed to process the last packet. cuLaunchHostFunc failed. Error: %d", Result);
					Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
					State = EReceptionState::FrameError;
					FrameErrorState(RTPHeader);

					CallbackPayload->bIsWaitingForPendingCopy.store(false, std::memory_order_release);
					return;
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxInputVideoStream::WaitingPendingOperation)
					const double CallbackTimestamp = FPlatformTime::Seconds();
					while (CallbackPayload->bIsWaitingForPendingCopy.load(std::memory_order_acquire) && bIsShuttingDown.load(std::memory_order_acquire) == false)
					{
						FPlatformProcess::SleepNoStats(UE::RivermaxCore::Private::Utils::SleepTimeSeconds);
						if (FPlatformTime::Seconds() - CallbackTimestamp > CVarWaitForCompletionTimeout.GetValueOnAnyThread())
						{
							UE_LOGF(LogRivermax, Error, "Waiting for gpu copy of sample timed out.");
							Listener->OnStreamError();
							CallbackPayload->bIsWaitingForPendingCopy.store(false, std::memory_order_release);
							break;
						}
					}
				}
#endif // PLATFORM_WINDOWS
			}

			FrameDescriptionTracking.ResetSingleFrameTracking();

			if (bIsShuttingDown.load(std::memory_order_acquire) == false)
			{
				Listener->OnVideoFrameReceived(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
				StreamData.CurrentFrameVideoBuffer = nullptr;
				StreamData.CurrentSamples.Empty();
				State = EReceptionState::Receiving;
			}
		}
		else
		{
			UE_LOGF(LogRivermax, Warning, "End of frame received but not enough data was received (missing %d). Expected %d but received (%d)", StreamData.ExpectedSize - StreamData.ReceivedSize, StreamData.ExpectedSize, StreamData.ReceivedSize);

			Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
			State = EReceptionState::FrameError;
			FrameErrorState(RTPHeader);

			++StreamStats.InvalidFramesCount;
		}
	}

	void FRivermaxInputVideoStream::FrameReceptionState(const FRTPHeader& RTPHeader, const uint8* DataPtr, TSharedPtr<IRivermaxSample> InSample)
	{
		const uint64 LastSequenceNumberIncremented = StreamData.LastSequenceNumber + 1;

		const uint64 LostPackets = ((uint64)RTPHeader.SequenceNumber + 0x100000000 - LastSequenceNumberIncremented) & 0xFFFFFFFF;
		if (LostPackets > 0)
		{
			UE_LOGF(LogRivermax, Warning, "Lost %llu packets during reception of chunk", LostPackets);

			StreamData.WritingOffset = 0;
			StreamData.ReceivedSize = 0;
			++StreamStats.TotalPacketLossCount;
			++StreamStats.FramePacketLossCount;

			Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
			State = EReceptionState::FrameError;
			FrameErrorState(RTPHeader);
			return;
		}

		StreamData.LastSequenceNumber = RTPHeader.SequenceNumber;

		ProcessSRD(RTPHeader, DataPtr);

		if (StreamData.ReceivedSize > StreamData.ExpectedSize)
		{
			UE_LOGF(LogRivermax, Warning, "Received too much data (%d). Expected %d but received (%d)", StreamData.ReceivedSize - StreamData.ExpectedSize, StreamData.ExpectedSize, StreamData.ReceivedSize);
			StreamData.WritingOffset = 0;
			StreamData.ReceivedSize = 0;
			++StreamStats.BiggerFramesCount;

			Listener->OnVideoFrameReceptionError(StaticCastSharedPtr<IRivermaxVideoSample>(InSample));
			State = EReceptionState::FrameError;
			FrameErrorState(RTPHeader);
			return;
		}

		if (RTPHeader.bIsMarkerBit)
		{
			ProcessLastSRD(RTPHeader, DataPtr, InSample);

			StreamStats.FramePacketLossCount = 0;
			++StreamStats.EndOfFrameReceived;
		}
	}

	void FRivermaxInputVideoStream::WaitForMarkerState(const FRTPHeader& RTPHeader)
	{
		if (RTPHeader.bIsMarkerBit)
		{
			StreamData.LastSequenceNumber = RTPHeader.SequenceNumber;
			StreamData.WritingOffset = 0;
			StreamData.ReceivedSize = 0;
			StreamData.DeviceWritePointerOne = nullptr;
			StreamData.SizeToWriteOne = 0;
			StreamData.DeviceWritePointerTwo = nullptr;
			StreamData.SizeToWriteTwo = 0;
			bIsFirstPacketReceived = false;

			StreamData.CurrentFrameVideoBuffer = nullptr;
			StreamData.CurrentSamples.Empty();
			State = EReceptionState::Receiving;
		}
	}

	void FRivermaxInputVideoStream::FrameErrorState(const FRTPHeader& RTPHeader)
	{
		WaitForMarkerState(RTPHeader);
	}

	void FRivermaxInputVideoStream::UpdateFrameTracking(const FRTPHeader& NewRTPHeader)
	{
		FrameDescriptionTracking.EvaluateNewRTP(NewRTPHeader);

		if (!Options.bEnforceVideoFormat)
		{
			if (FrameDescriptionTracking.DetectedResolution.X > 0 && FrameDescriptionTracking.DetectedResolution.Y > 0)
			{
				if (FrameDescriptionTracking.DetectedResolution != StreamResolution)
				{
					StreamResolution = FrameDescriptionTracking.DetectedResolution;

					FRivermaxInputVideoFormatChangedInfo FormatChangeInfo;
					FormatChangeInfo.Width = StreamResolution.X;
					FormatChangeInfo.Height = StreamResolution.Y;
					FormatChangeInfo.PixelFormat = FrameDescriptionTracking.ExpectedSamplingType;
					Listener->OnVideoFormatChanged(FormatChangeInfo);
				}
			}
		}
	}

	bool FRivermaxInputVideoStream::InitializeStreamParameters(FString& OutErrorMessage)
	{
		size_t BufferElement = 1 << 18;
		CachedAPI->rmx_input_set_mem_capacity_in_packets(&StreamParameters, BufferElement);

		constexpr size_t SubBlockCount = 2;
		CachedAPI->rmx_input_set_mem_sub_block_count(&StreamParameters, SubBlockCount);
		CachedAPI->rmx_input_set_entry_size_range(&StreamParameters, BufferConfiguration.HeaderBlockID, BufferConfiguration.HeaderExpectedSize, BufferConfiguration.HeaderExpectedSize);
		CachedAPI->rmx_input_set_entry_size_range(&StreamParameters, BufferConfiguration.DataBlockID, ExpectedPayloadSize, ExpectedPayloadSize);

		constexpr rmx_input_option InputOptions = RMX_INPUT_STREAM_CREATE_INFO_PER_PACKET;
		constexpr rmx_input_timestamp_format TimestampFormat = RMX_INPUT_TIMESTAMP_RAW_NANO;
		CachedAPI->rmx_input_enable_stream_option(&StreamParameters, InputOptions);
		CachedAPI->rmx_input_set_timestamp_format(&StreamParameters, TimestampFormat);

		if (bIsDynamicHeaderEnabled)
		{
			CachedAPI->rmx_input_enable_stream_option(&StreamParameters, RMX_INPUT_STREAM_RTP_SMPTE_2110_20_DYNAMIC_HDS);
		}

		const rmx_status Status = CachedAPI->rmx_input_determine_mem_layout(&StreamParameters);
		if (Status != RMX_OK)
		{
			OutErrorMessage = FString::Printf(TEXT("Could not determine memory layout for input stream using IP %s. Status: %d."), *Options.InterfaceAddress, Status);
			return false;
		}

		BufferElement = CachedAPI->rmx_input_get_mem_capacity_in_packets(&StreamParameters);
		BufferConfiguration.DataMemory = CachedAPI->rmx_input_get_mem_block_buffer(&StreamParameters, BufferConfiguration.DataBlockID);
		BufferConfiguration.HeaderMemory = CachedAPI->rmx_input_get_mem_block_buffer(&StreamParameters, BufferConfiguration.HeaderBlockID);

		if (BufferConfiguration.HeaderMemory->length <= 0)
		{
			OutErrorMessage = FString::Printf(TEXT("Header data split not supported for device with IP %s. Can't initialize stream."), *Options.InterfaceAddress);
			return false;
		}

		BufferConfiguration.PayloadSize = BufferConfiguration.DataMemory->length;
		BufferConfiguration.HeaderSize = BufferConfiguration.HeaderMemory->length;

		AllocateBuffers();

		return true;
	}
}
