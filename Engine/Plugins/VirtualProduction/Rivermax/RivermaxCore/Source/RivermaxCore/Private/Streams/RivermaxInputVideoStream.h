// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RivermaxInStream.h"

#include "RHI.h"
#include "RivermaxTypes.h"

namespace UE::RivermaxCore::Private
{
	using UE::RivermaxCore::IRivermaxInputStream;
	using UE::RivermaxCore::IRivermaxInputStreamListener;
	using UE::RivermaxCore::FRivermaxInputStreamOptions;

	struct FRawRTPHeader;
	struct FRTPHeader;

	struct FInputStreamBufferConfiguration
	{
		size_t PayloadSize = 0;
		size_t HeaderSize = 0;
		uint16 PayloadExpectedSize = 1500;
		uint16 HeaderExpectedSize = 32;

		/** Data Sub block ID */
		const uint8 HeaderBlockID = 0;

		/** Data Sub block ID */
		const uint8 DataBlockID = 1;

		/** Memory region used to store incoming payloads */
		rmx_mem_region* DataMemory = nullptr;

		/** Payload stride in bytes for input stream */
		size_t DataStride = 0;

		/** Memory region used to store incoming headers */
		rmx_mem_region* HeaderMemory = nullptr;

		/** Header stride in bytes for input stream */
		size_t HeaderStride = 0;
	};

	struct FInputStreamStats
	{
		uint64 InvalidHeadercount = 0;
		uint64 FramePacketLossCount = 0;
		uint64 TotalPacketLossCount = 0;
		uint64 BiggerFramesCount = 0;
		uint64 InvalidFramesCount = 0;
		uint64 BytesReceived = 0;
		uint64 FramesReceived = 0;
		uint64 ChunksReceived = 0;
		uint64 EndOfFrameReceived = 0;
		uint64 EmptyCompletionCount = 0;
		uint64 MinFirstPacketIntervalNS = TNumericLimits<uint64>::Max();
		uint64 MaxFirstPacketIntervalNS = TNumericLimits<uint64>::Min();
		uint64 FirstPacketIntervalAccumulatorNS = 0;
		uint64 FirstPacketIntervalStatsCount = 0;
	};

	struct FInputStreamData
	{
		uint64 LastSequenceNumber = 0;
		void* CurrentFrameVideoBuffer = nullptr;
		uint32 WritingOffset = 0;
		uint32 ReceivedSize = 0;
		uint32 ExpectedSize = 0;

		// GPUDirect data handling. Rivermax uses a cyclic buffer so we might have part of the frame at the end of the first buffer and the rest on the second part.
		const void* DeviceWritePointerOne = nullptr;
		const void* DeviceWritePointerTwo = nullptr;
		uint32 SizeToWriteOne = 0;
		uint32 SizeToWriteTwo = 0;

		// A map of samples for the current frame.
		TMap<IRivermaxSample::ESampleType, TSharedPtr<IRivermaxSample>> CurrentSamples;
	};

	/** Incoming frame description tracker. Used to detect variable SRD length across a frame for now but should be used to help auto detect incoming frame */
	struct FFrameDescriptionTrackingData
	{
	public:

		/** Resets data related to current frame being tracked */
		void ResetSingleFrameTracking();

		/** Look at new RTP and extract info to estimate frame description */
		void EvaluateNewRTP(const FRTPHeader& NewHeader);

		/** Whether a valid resolution was detected during the last frame */
		bool HasDetectedValidResolution() const;

	private:

		/** Updates current resolution detection using new RTP header. */
		void UpdateResolutionDetection(const FRTPHeader& NewHeader);

		/** Updates payload sizes encountered throughout a frame */
		void UpdatePayloadSizeTracking(const FRTPHeader& NewHeader);

	public:
		/** Used to track different payload sizes received. GPUDirect can't work if different payload sizes are received across the frame except the last one */
		TMap<uint32, uint32> PayloadSizeReceived;

		/** Payload size used across a frame. -1 if multi SRD except last one are received. */
		int16 CommonPayloadSize = -1;

		/** Last resolution detected */
		FIntPoint DetectedResolution = FIntPoint::ZeroValue;

		/** Sampling type we are expecting as setup by the user. We could also auto detect it but some sampling types would collide so left manual for now. */
		ESamplingType ExpectedSamplingType = ESamplingType::RGB_10bit;

		/** Used to log sampling type warning once until a good frame type is received */
		bool bHasLoggedSamplingWarning = false;
	};

	class FRivermaxInputVideoStream : public FRivermaxInStream
	{
	public:
		FRivermaxInputVideoStream();
		virtual ~FRivermaxInputVideoStream();

	protected:
		//~ Begin FRivermaxInStream interface
		virtual void OnPreInitialize(IRivermaxCoreModule& RivermaxModule) override;
		virtual void OnPreThreadStop() override;
		virtual void OnPostThreadStop() override;
		virtual FString GetThreadName() const override;
		virtual bool GetGPUDirectSupported() const override;
		virtual bool InitializeStreamParameters(FString& OutErrorMessage) override;
		virtual void OnStreamCreated() override;
		virtual size_t GetMaxCompletionChunkSize() const override;
		virtual void DeallocateBuffers() override;
		virtual void ParseChunks(const rmx_input_completion* Completion) override;
		virtual void PrintStats() override;
		virtual void LogStreamDescriptionOnCreation() const override;
		//~ End FRivermaxInStream interface

	private:

		/** Process reception state. i.e Extract RTP, tracks data received, etc... */
		void FrameReceptionState(const FRTPHeader& RTPHeader, const uint8* DataPtr, TSharedPtr<IRivermaxSample> InSample);

		/** State where we just wait to receive an end of frame in RTP. Used to align for the first frame and when an error occurs during reception */
		void WaitForMarkerState(const FRTPHeader& RTPHeader);

		/**
		 * State we go in when an error was detected during the reception of a frame.
		 * Once the end of frame is received, will go back to frame reception state.
		 */
		void FrameErrorState(const FRTPHeader& RTPHeader);

		/** Process a SRD / packet by copying data associated to it into output buffer or tracking memory received for gpudirect */
		void ProcessSRD(const FRTPHeader& RTPHeader, const uint8* DataPtr);

		/** Handles wrapping up of a frame when marker bit is detected (last SRD). Will notify listener of a new frame received */
		void ProcessLastSRD(const FRTPHeader& RTPHeader, const uint8* DataPtr, TSharedPtr<IRivermaxSample> InSample);

		/** Requests a new frame from our listener. If it fails, it will fall in error state waiting for the next one */
		TSharedPtr<IRivermaxVideoSample> GetVideoSampleForReception(const FRTPHeader& RTPHeader);

		/** Allocates system or GPU memory buffer used for reception */
		void AllocateBuffers();

		/** Allocates gpu memory buffer used for reception and used by rivermax */
		bool AllocateGPUBuffers();

		/** Gets the mapped gpu buffer for the incoming RHI buffer. Caching will occur if a new buffer is given */
		void* GetMappedBuffer(const FBufferRHIRef& InBuffer);

		/** Updates frame information tracking using RTP header */
		void UpdateFrameTracking(const FRTPHeader& NewRTPHeader);

	private:

		/** Memory configuration used for this stream. */
		FInputStreamBufferConfiguration BufferConfiguration;

		/** Used to know when the first packet of a frame has been received. Used mostly for visualization in insights */
		bool bIsFirstPacketReceived = false;

		/** Information about the incoming frame we are receiving and general sequence numbering we are using for packet loss */
		FInputStreamData StreamData;

		/** Various stats kept for this stream */
		FInputStreamStats StreamStats;

		/** Info about the video format that was selected such as pixel group, group size etc */
		FVideoFormatInfo FormatInfo;

		/** Whether gpudirect is used for this stream. Even though listener requests it, doesn't mean it will/can be used */
		bool bIsUsingGPUDirect = false;

		/** Whether this stream has dynamic header data split enabled. Need to disable it on shutdown */
		bool bIsDynamicHeaderEnabled = false;

		/** GPU device index to use when mapping to cuda. Will be needed for mgpu support */
		int32 GPUDeviceIndex = 0;

		/** Allocated memory base address used when it's time to free */
		void* GPUAllocatedMemoryBaseAddress = nullptr;

		/** Total allocated gpu memory. */
		size_t GPUAllocatedMemorySize = 0;

		/** Used to store all the required information for mapping GPU buffers */
		struct FCudaExternalBufferMapping
		{
			// External memory handle created from RHI buffer to be mapped with cuda.
			void* ExternalMemory = nullptr;

			// Pointer (CUdeviceptr) to mapped external memory.
			void* MappedPtr = nullptr;
		};

		/** Map between buffer we are writing to and their mapped address in cuda space */
		TMap<FRHIBuffer*, FCudaExternalBufferMapping> BufferGPUMemoryMap;

		/** Used to wait for a frame to be copied from gpu to gpu memory */
		struct FCallbackPayload
		{
			std::atomic<bool> bIsWaitingForPendingCopy = false;
		};
		TSharedPtr<FCallbackPayload> CallbackPayload;

		/** Cuda stream used for our operations */
		void* GPUStream = nullptr;

		/** Payload size used to initialize rivermax stream and know differences between received SRD and config */
		uint32 ExpectedPayloadSize = 1500;

		/**
		 * States used to track reception of a frame
		 */
		enum class EReceptionState
		{
			/** Used for the first frame receptionand when an error happened while receiving to detect reception of the end of frame */
			WaitingForMarker,

			/** Used while receiving a frame and processing each SampleRowData */
			Receiving,

			/** Used when an error is detected during reception. Will wait for the end of frame detection before moving back to Receiving */
			FrameError
		};

		/** Starts stream waiting for a marker to align ourselves with a full frame */
		EReceptionState State = EReceptionState::WaitingForMarker;

		/** Track incoming frame description to infer resolution, pixel type. */
		FFrameDescriptionTrackingData FrameDescriptionTracking;

		/** Resolution being used by this stream. Can either be auto detected or enforced by owner */
		FIntPoint StreamResolution = FIntPoint::ZeroValue;
	};
}
