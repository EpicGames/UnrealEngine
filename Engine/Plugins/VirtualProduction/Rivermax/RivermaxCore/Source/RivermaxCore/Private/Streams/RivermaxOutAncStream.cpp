// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxOutAncStream.h"

#include "Async/Async.h"
#if PLATFORM_WINDOWS
#include "CudaModule.h"
#endif // PLATFORM_WINDOWS
#include "IRivermaxCoreModule.h"
#include "IRivermaxBoundaryMonitor.h"
#include "IRivermaxManager.h"
#include "Misc/ByteSwap.h"
#include "RivermaxFrameAllocator.h"
#include "RivermaxFrameManager.h"
#include "RivermaxLog.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxTracingUtils.h"
#include "RivermaxTypes.h"
#include "RivermaxUtils.h"



#define GET_FRAME_INDEX(OUTPUT_FRAME) OUTPUT_FRAME->GetFrameCounter() % Options.NumberOfBuffers
namespace UE::RivermaxCore::Private
{
	/**
	* Generares a 10 bit Data Count based on the number of User Data Words. Requirement by RFC8331:
	* The lower 8 bits of Data_Count, corresponding to bits b7 (MSB; most significant bit) through b0 (LSB; least
    * significant bit) of the 10-bit Data_Count word, contain the actual count of 10-bit words in User_Data_Words.  Bit b8 is
	* the even parity for bits b7 through b0, and bit b9 is the inverse (logical NOT) of bit b8.
	*/
	uint16 EncodeDataCount(uint8 UDWCount)
	{
		uint8 Parity = 0;
		uint8 Temp = UDWCount;

		for (int32 i = 0; i < 8; ++i)
		{
			Parity ^= (Temp & 1);
			Temp >>= 1;
		}

		uint8 B8 = Parity;
		uint8 B9 = ~B8 & 0x01;

		uint16 DataCount = (B9 << 9) | (B8 << 8) | UDWCount;
		return DataCount;
	}

	/**
	* Fills the Rivermax memory with ANC header and data.
	* From https://datatracker.ietf.org/doc/html/rfc8331
	*        0                   1                   2                   3
    *   0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1 2 3 4 5 6 7 8 9 0 1
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |V=2|P|X| CC    |M|    PT       |        sequence number        |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |                           timestamp                           |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |           synchronization source (SSRC) identifier            |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |   Extended Sequence Number    |           Length=32           |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   | ANC_Count=2   | F |                reserved                   |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |C|   Line_Number=9     |   Horizontal_Offset   |S| StreamNum=0 |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |         DID       |        SDID       |  Data_Count=0x84  |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *                            User_Data_Words...
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *               |   Checksum_Word   |         word_align            |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |C|   Line_Number=10    |   Horizontal_Offset   |S| StreamNum=0 |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *   |         DID       |        SDID       |  Data_Count=0x105 |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *                            User_Data_Words...
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
    *                                   |   Checksum_Word   |word_align |
    *   +-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+-+
	*
    *         Figure 1: SMPTE Ancillary Data RTP Packet Format
	*/
	void FillChunk
		( const TSharedPtr<FRivermaxAncOutputOptions> InStreamOptions
		, const FRivermaxOutputStreamMemory& InStreamMemory
		, uint8* InFirstPacketStartPointer
		, uint16* InPayloadSizes
		, const TArray<uint16>& UDWs
		, const UE::RivermaxCore::Private::FRivermaxOutputStreamData& StreamData
		, TSharedPtr<UE::RivermaxCore::Private::FRivermaxOutputFrame> CurrentFrame)
	{
		// TODO: Infer from SDP
		constexpr uint8 PayloadType = 97;

		// Anc field values.
		constexpr uint8 ProgressiveField = 0b00;
		constexpr uint8 InterlaceField = 0b10;
		constexpr uint8 InterlaceSecondField = 0b11;
		constexpr int32 Zero = 0;
		static bool bOnce = false;

		for (size_t PayloadIndex = 0; PayloadIndex < InStreamMemory.PacketsPerChunk; ++PayloadIndex) 
		{
			int64 PacketByte = PayloadIndex * InStreamMemory.PayloadSize;
			FBigEndianHeaderPacker HeaderPacker(InFirstPacketStartPointer + PacketByte, InStreamMemory.PayloadSize, true /*bClearExisting*/);
			constexpr uint8 Version = 2;
			HeaderPacker.AddField(Version, 2 RIVERMAX_DEBUG_FIELD_NAME("V"));

			HeaderPacker.AddField(Zero, 1 RIVERMAX_DEBUG_FIELD_NAME("P"));
			HeaderPacker.AddField(Zero, 1 RIVERMAX_DEBUG_FIELD_NAME("X"));

			constexpr uint8 ContributingSourceCount = 0;
			HeaderPacker.AddField(ContributingSourceCount, 4 RIVERMAX_DEBUG_FIELD_NAME("CC"));

			// Only one packet for ANC, which means first packet is the last packet.
			const uint8 MarkerBit = 1;
			HeaderPacker.AddField(MarkerBit, 1 RIVERMAX_DEBUG_FIELD_NAME("M"));

			HeaderPacker.AddField(PayloadType, 7 RIVERMAX_DEBUG_FIELD_NAME("PT"));

			HeaderPacker.AddField(StreamData.SequenceNumber & 0xFFFF, 16 RIVERMAX_DEBUG_FIELD_NAME("SEQ"));
			HeaderPacker.AddField(CurrentFrame->MediaTimestamp, 32 RIVERMAX_DEBUG_FIELD_NAME("Timestamp"));
			HeaderPacker.AddField(StreamData.SynchronizationSource, 32 RIVERMAX_DEBUG_FIELD_NAME("SSRC"));
			HeaderPacker.AddField((StreamData.SequenceNumber >> 16) & 0xFFFF, 16 RIVERMAX_DEBUG_FIELD_NAME("Extended Sequence Number"));

			const int32 LengthBitPosition = HeaderPacker.CountFieldBits();

			// Will be overwritten later.
			HeaderPacker.AddField(0, 16 RIVERMAX_DEBUG_FIELD_NAME("Length"));

			constexpr uint8 ANCCount = 1;
			HeaderPacker.AddField(ANCCount, 8 RIVERMAX_DEBUG_FIELD_NAME("ANC Count"));
			HeaderPacker.AddField(ProgressiveField, 2 RIVERMAX_DEBUG_FIELD_NAME("F"));
			// Padding - always zero.
			HeaderPacker.AddField(Zero, 22 RIVERMAX_DEBUG_FIELD_NAME("Reserved"));

			// Length field in the packet starts counting bytes from field "C" up until and including the WordAlign that is done later.
			const int32 LengthCountStart = HeaderPacker.CountBytes();

			HeaderPacker.AddField(Zero, 1 RIVERMAX_DEBUG_FIELD_NAME("C"));

			// rfc8331 - Without specific line location within the field.
			constexpr int32 LineNumber = 0x7FF;
			HeaderPacker.AddField(LineNumber, 11 RIVERMAX_DEBUG_FIELD_NAME("Line Number"));

			// ANC data packet generic horizontal location per rfc8331:
			// 0xFFF - Without specific horizontal location
			constexpr int32 HorizontalOffset = 0xFFF;
			HeaderPacker.AddField(HorizontalOffset, 12 RIVERMAX_DEBUG_FIELD_NAME("Horizontal Offset"));

			HeaderPacker.AddField(Zero, 1 RIVERMAX_DEBUG_FIELD_NAME("S"));

			constexpr uint8 StreamNum = 0;
			HeaderPacker.AddField(StreamNum, 7 RIVERMAX_DEBUG_FIELD_NAME("StreamNum"));

			// MakeDataIdentificationWord
			HeaderPacker.AddField(InStreamOptions->GetDID(), 10 RIVERMAX_DEBUG_FIELD_NAME("DID"));
			HeaderPacker.AddField(InStreamOptions->GetSDID(), 10 RIVERMAX_DEBUG_FIELD_NAME("SDID"));

			// UDW.Num() will never exceed 255 since that is max per ANC packet.
			constexpr uint32 MaxNumUDW = 255;
			check(UDWs.Num() <= MaxNumUDW);
			const int16 DataCount = EncodeDataCount(UDWs.Num());
			HeaderPacker.AddField(DataCount, 10 RIVERMAX_DEBUG_FIELD_NAME("Data_Count"));

			// UDW must already have parity bits written into them.
			for (int32 UDWIndex = 0; UDWIndex < UDWs.Num(); ++UDWIndex)
			{
				FString Label = FString::Printf(TEXT("DWORD%d"), UDWIndex + 1);
				HeaderPacker.AddField(UDWs[UDWIndex], 10 RIVERMAX_DEBUG_FIELD_NAME(*Label));
			}

			const uint16 Checksum = UE::RivermaxCore::Private::Utils::ComputeAncChecksum(
				InStreamOptions->GetDID(),
				InStreamOptions->GetSDID(),
				DataCount,
				UDWs
			);			
			
			HeaderPacker.AddField(Checksum, 10 RIVERMAX_DEBUG_FIELD_NAME("Checksum"));
			HeaderPacker.WordAlign(sizeof(int32));

			HeaderPacker.Finalize();
			const int32 LengthCountEnd = HeaderPacker.CountBytes();

			check(LengthCountEnd < InStreamMemory.PayloadSize);

			HeaderPacker.UpdateField(LengthCountEnd - LengthCountStart, 16, LengthBitPosition RIVERMAX_DEBUG_FIELD_NAME("Length"));

			InPayloadSizes[PayloadIndex] = LengthCountEnd;
		}
	}


	FRivermaxOutputStreamAnc::FRivermaxOutputStreamAnc(const TArray<char>& SDPDescription)
		: FRivermaxOutStream(SDPDescription)
		, FrameInfoToSend(MakeShared<FRivermaxOutputInfoAnc>())
	{
		StreamType = ERivermaxStreamType::ST2110_40;
	}

	bool FRivermaxOutputStreamAnc::PushFrame(TSharedPtr<IRivermaxOutputInfo> InFrameInfo)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutputStreamAnc::PushVideoFrame);

		// Clear reserved frame if there is one. If not, try to get free frame.
		// ReservedFrame should always be valid when block on reservation mode is used.
		// At the moment we always have reserved frame for ANC
		check(ReservedFrames.RemoveAndCopyValue(InFrameInfo->FrameIdentifier, NextFrameToSend));
		FrameInfoToSend = StaticCastSharedPtr<FRivermaxOutputInfoAnc>(InFrameInfo);
		FrameReadyToSendSignal->Trigger();
		return true;
	}

	bool FRivermaxOutputStreamAnc::InitializeStreamMemoryConfig()
	{
		using namespace UE::RivermaxCore::Private::Utils;

		constexpr int64 PacketsPerFrame = 1;

		// We might need a smaller packet to complete the end of frame so ceil to the next value
		StreamMemory.PacketsPerFrame = PacketsPerFrame;

		constexpr size_t StridesInChunk = 1;
		StreamMemory.PacketsPerChunk = StridesInChunk;
		StreamMemory.FramesFieldPerMemoryBlock = CachedCVars.bUseSingleMemblock ? Options.NumberOfBuffers : 1;

		// Only one chunk with one packet.
		constexpr int64 ChunksPerFrame = 1.;
		StreamMemory.ChunksPerFrameField = ChunksPerFrame;
		const uint64 RealPacketsPerFrame = StreamMemory.ChunksPerFrameField * StreamMemory.PacketsPerChunk;
		StreamMemory.PacketsPerMemoryBlock = RealPacketsPerFrame * StreamMemory.FramesFieldPerMemoryBlock;
		StreamMemory.ChunksPerMemoryBlock = StreamMemory.FramesFieldPerMemoryBlock * StreamMemory.ChunksPerFrameField;
		StreamMemory.MemoryBlockCount = Options.NumberOfBuffers / StreamMemory.FramesFieldPerMemoryBlock;
		StreamMemory.DataBlockID = 0;
		// A size close to max UDP packet size. Since it is only one packet.
		StreamMemory.PayloadSize = 1300;

		StreamMemory.ChunkSpacingBetweenMemcopies = 1;
		StreamMemory.bUseIntermediateBuffer = true;

		if (!SetupFrameManagement())
		{
			return false;
		}

		StreamMemory.MemoryBlocks.SetNumZeroed(StreamMemory.MemoryBlockCount);
		CachedAPI->rmx_output_media_init_mem_blocks(StreamMemory.MemoryBlocks.GetData(), StreamMemory.MemoryBlockCount);
		for (uint32 BlockIndex = 0; BlockIndex < StreamMemory.MemoryBlockCount; ++BlockIndex)
		{
			rmx_output_media_mem_block& Block = StreamMemory.MemoryBlocks[BlockIndex];
			CachedAPI->rmx_output_media_set_chunk_count(&Block, StreamMemory.ChunksPerMemoryBlock);

			// Anc only needs one sub block since we don't split header and data.
			constexpr uint8 SubBlockCount = 1;
			CachedAPI->rmx_output_media_set_sub_block_count(&Block, SubBlockCount);
		}
		
		return true;
	}

	bool FRivermaxOutputStreamAnc::IsFrameAvailableToSend()
	{
		// This should also depend on Video stream.
		return true;
	}

	bool FRivermaxOutputStreamAnc::CopyFrameData(const TSharedPtr<FRivermaxOutputFrame>& SourceFrame, uint8* DestinationBase)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutputStreamAnc::CopyFrameData);

		const uint32 MediaFrameNumber = Utils::TimestampToFrameNumber(CurrentFrame->MediaTimestamp, GetFrameRate());
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutAncSendingFrameTraceEvents[MediaFrameNumber % 10]);

		TSharedPtr<FRivermaxAncOutputOptions> StreamOptions = StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[(uint8)StreamType]);

		// Alternative to rmx_output_media_set_packet_layout. Required for dynamically sized packets.
		CachedAPI->rmx_output_media_set_chunk_packet_count(&StreamData.ChunkHandle, (size_t)StreamMemory.PacketsPerChunk);

		// This size will be filled with the actual sizes of the packets.
		uint16* PayloadSizes = rmx_output_media_get_chunk_packet_sizes(&StreamData.ChunkHandle, StreamMemory.DataBlockID);

		FillChunk(StreamOptions, StreamMemory, reinterpret_cast<uint8*>(CurrentFrame->FrameStartPtr), PayloadSizes, FrameInfoToSend->UDWs, StreamData, CurrentFrame);

		OnFrameReadyToBeSent();

		return true;
	}

	bool FRivermaxOutputStreamAnc::SetupFrameManagement()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutputStreamAnc::SetupFrameManagement);

		return true;
	}

	void FRivermaxOutputStreamAnc::CleanupFrameManagement()
	{
		FRivermaxOutStream::CleanupFrameManagement();
	}

	bool FRivermaxOutputStreamAnc::ReserveFrame(uint64 FrameCounter) const
	{
		// There is only one reserved frame at the time per stream.
		TSharedPtr<FRivermaxOutputFrame> ReservedFrame = MakeShared<FRivermaxOutputFrame>();

		if (ReservedFrame.IsValid())
		{
			ReservedFrame->SetFrameCounter(FrameCounter);
			ReservedFrames.Add(FrameCounter, ReservedFrame);
		}

		return ReservedFrame.IsValid();
	}

	TSharedPtr<FRivermaxOutputFrame> FRivermaxOutputStreamAnc::GetNextFrameToSend(bool bWait)
	{
		if (bWait)
		{
			while (!NextFrameToSend.IsValid() && bIsActive)
			{
				FrameReadyToSendSignal->Wait();
			}
		}
		return NextFrameToSend;
	}

	void FRivermaxOutputStreamAnc::LogStreamDescriptionOnCreation() const
	{
		FRivermaxOutStream::LogStreamDescriptionOnCreation();

		TStringBuilder<512> StreamDescription;

		TSharedPtr<FRivermaxAncOutputOptions> StreamOptions = StaticCastSharedPtr<FRivermaxAncOutputOptions>(Options.StreamOptions[(uint8)StreamType]);
		StreamDescription.Appendf(TEXT("FrameRate = %s, "), *StreamOptions->FrameRate.ToPrettyText().ToString());
		StreamDescription.Appendf(TEXT("Alignment = %s, "), LexToString(Options.AlignmentMode));
		StreamDescription.Appendf(TEXT("Framelocking = %s."), LexToString(Options.FrameLockingMode));

		UE_LOGF(LogRivermax, Display, "%ls", *FString(StreamDescription));
	}

	void FRivermaxOutputStreamAnc::SetupRTPHeadersForChunk()
	{
		// ANC RTP headers are shipped with the payload. All needs to be done is incrementation of packets and sequence number.
		++StreamData.SequenceNumber;
	}

	void FRivermaxOutputStreamAnc::CompleteCurrentFrame(bool bReleaseFrame)
	{
		FRivermaxOutStream::CompleteCurrentFrame(bReleaseFrame);
	}

	FRivermaxOutputStreamAncTimecode::FRivermaxOutputStreamAncTimecode(const TArray<char>& SDPDescription)
		: FRivermaxOutputStreamAnc(SDPDescription)
	{
		StreamType = ERivermaxStreamType::ST2110_40_TC;
	}

	bool FRivermaxOutputStreamAncTimecode::PushFrame(TSharedPtr<IRivermaxOutputInfo> InFrameInfo)
	{
		TSharedPtr<FRivermaxOutputInfoAncTimecode> FrameInfo = StaticCastSharedPtr<FRivermaxOutputInfoAncTimecode>(InFrameInfo);
		FrameInfo->UDWs = UE::RivermaxCore::Private::Utils::TimecodeToAtcUDW10(FrameInfo->Timecode, FrameInfo->FrameRate);
		
		return FRivermaxOutputStreamAnc::PushFrame(FrameInfo);
	}
}

