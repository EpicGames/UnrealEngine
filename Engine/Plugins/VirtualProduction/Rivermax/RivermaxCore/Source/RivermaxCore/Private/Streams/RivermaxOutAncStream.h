// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RivermaxOutStream.h"

#include "Async/Future.h"
#include "Containers/SpscQueue.h"
#include "HAL/Runnable.h"
#include "RivermaxWrapper.h"
#include "RivermaxOutputFrame.h"
#include "RivermaxTypes.h"
#include "RTPHeader.h"


class FEvent;
class IRivermaxCoreModule;

namespace UE::RivermaxCore::Private
{
	using UE::RivermaxCore::FRivermaxOutputOptions;

	/** Generic ANC. */
	class FRivermaxOutputStreamAnc : public FRivermaxOutStream
	{
	public:
		FRivermaxOutputStreamAnc(const TArray<char>& SDPDescription);

	public:

		//~ Begin IRivermaxOutputStream interface
		virtual bool PushFrame(TSharedPtr<IRivermaxOutputInfo> FrameInfo) override;
		virtual bool ReserveFrame(uint64 FrameCounter) const override;
		//~ End IRivermaxOutputStream interface

	protected:

		//~ Begin FRivermaxOutStream interface. Refer to the parent class for more information.
		virtual bool IsFrameAvailableToSend();
		virtual bool InitializeStreamMemoryConfig();
		virtual bool CopyFrameData(const TSharedPtr<FRivermaxOutputFrame>& SourceFrame, uint8* DestinationBase);
		virtual bool SetupFrameManagement();
		virtual void CleanupFrameManagement() override;
		virtual void CompleteCurrentFrame(bool bReleaseFrame) override;
		virtual void SetupRTPHeadersForChunk() override;
		virtual void LogStreamDescriptionOnCreation() const override;
		virtual TSharedPtr<FRivermaxOutputFrame> GetNextFrameToSend(bool bWait = false);
		//~ End FRivermaxOutStream interface

	private:

		/** All the information required for ANC data to be sent. */
		TSharedPtr<FRivermaxOutputInfoAnc> FrameInfoToSend;

		/** Next frame to send. */
		TSharedPtr<FRivermaxOutputFrame> NextFrameToSend;
	};

	/** Ancillary Timecode. */
	class FRivermaxOutputStreamAncTimecode : public FRivermaxOutputStreamAnc
	{
	public:
		FRivermaxOutputStreamAncTimecode(const TArray<char>& SDPDescription);
	public:

		//~ Begin IRivermaxOutputStream interface
		virtual bool PushFrame(TSharedPtr<IRivermaxOutputInfo> FrameInfo) override;
		//~ End IRivermaxOutputStream interface
	};
}


