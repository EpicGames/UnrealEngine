// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxOutStream.h"

#include "Async/Async.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxBoundaryMonitor.h"
#include "IRivermaxManager.h"
#include "Misc/ByteSwap.h"
#include "RivermaxFrameAllocator.h"
#include "RivermaxFrameManager.h"
#include "RivermaxLog.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxTracingUtils.h"
#include "RivermaxTypes.h"
#include "RivermaxUtils.h"

#if PLATFORM_WINDOWS
#include "CudaModule.h"
#elif PLATFORM_LINUX
#include <pthread.h>
#include <sched.h>
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX

#define GET_FRAME_INDEX(OUTPUT_FRAME) OUTPUT_FRAME->GetFrameCounter() % Options.NumberOfBuffers
namespace UE::RivermaxCore::Private
{
	static TAutoConsoleVariable<int32> CVarRivermaxWakeupOffset(
		TEXT("Rivermax.WakeupOffset"), 0,
		TEXT("Wakeup is done on alignment point. This offset will be substracted from it to wake up earlier. Units: nanoseconds"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxScheduleOffset(
		TEXT("Rivermax.ScheduleOffset"), 0,
		TEXT("Scheduling is done at alignment point plus TRO. This offset will be added to it to delay or schedule earlier. Units: nanoseconds"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputShowStats(
		TEXT("Rivermax.ShowOutputStats"), 0,
		TEXT("Enable stats logging at fixed interval"),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarRivermaxOutputShowStatsInterval(
		TEXT("Rivermax.ShowStatsInterval"), 1.0,
		TEXT("Interval at which to show stats in seconds"),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarRivermaxOutputTROOverride(
		TEXT("Rivermax.Output.TRO"), 0,
		TEXT("If not 0, overrides transmit offset calculation (TRO) based on  frame rate and resolution with a fixed value. Value in seconds."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputSkipSchedulingCutOffTime(
		TEXT("Rivermax.Output.Scheduling.SkipCutoff"), 50,
		TEXT("Required time in microseconds from scheduling time to avoid skipping an interval."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputForceImmediateSchedulingThreshold(
		TEXT("Rivermax.Output.Scheduling.ForceImmediateCutoff"), 600,
		TEXT("Required time in nanoseconds from scheduling time before we clamp to do it immediately."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputUseSingleMemblock(
		TEXT("Rivermax.Output.UseSingleMemblock"), 1,
		TEXT("Configures Rivermax stream to use a single memblock potentially improving SDK performance."),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarRivermaxOutputRandomDelay(
		TEXT("Rivermax.Output.TriggerRandomDelayMs"), 0.0,
		TEXT("Will cause a delay of selected amount of time when next frame is sent in milliseconds."),
		ECVF_Default);

	enum class ETimingIssueMode : int32
	{
		Disabled = 0,
		Immediate = 1,
		Delay = 2,
		RandomContinuous = 3,
	};

	static TAutoConsoleVariable<int32> CVarRivermaxOutputTriggerTimingIssue(
		TEXT("Rivermax.Output.TriggerTimingIssue"), 0,
		TEXT("This is a debugging Cvar to introduce artificial timing issue.\n\
		0: disabled \n\
		1: set to this value to do it once, \n\
		2: set to this value to skip one frame in rougly 10 seconds.\n\
		3: set to this value to continuosly and randomly skip 10% of the frames."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputEnableTimingProtection(
		TEXT("Rivermax.Output.Scheduling.EnableTimingProtection"), 1,
		TEXT("Whether timing verification is done on commit to avoid misalignment. Next frame interval is skipped if it happens."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputEnableTimeCriticalThread(
		TEXT("Rivermax.Output.EnableTimeCriticalThread"), 0,
		TEXT("Whether to set output thread as time critical."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarRivermaxOutputPrefillRTPHeaders(
		TEXT("Rivermax.Output.PrefillRTPHeaders"), true,
		TEXT("Optimization used to prefill every RTP headers with known data."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarRivermaxOutputTrackChunkCompletion(
		TEXT("Rivermax.Output.TrackChunkCompletion"), false,
		TEXT("If true Rivermax Plugin will track when certain chunks (such as first chunk in the frame) were committed to the wire."),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarRivermaxOutputCommitChunksOffsetPercent(
		TEXT("Rivermax.Output.CommitChunksOffsetPercent"), 0.3f,
		TEXT("This CVar will allow for chunks to be committed before the next alignment point if it is at all possible.\n\
		The value indicates in percent of frame time how much earlier the Rivermax plugin will attempt to commit chunks."),
		ECVF_Default);



	FRivermaxOutStream::FRivermaxOutStream(const TArray<char>& InSDPDescription)
		: bIsActive(false)
		, SDPDescription(InSDPDescription)
	{

	}

	FRivermaxOutStream::~FRivermaxOutStream()
	{
		Uninitialize();
	}

	bool FRivermaxOutStream::Initialize(const FRivermaxOutputOptions& InOptions, IRivermaxOutputStreamListener& InListener)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutStream::Initialize);

		RivermaxModule = FModuleManager::GetModulePtr<IRivermaxCoreModule>(TEXT("RivermaxCore"));
		if (RivermaxModule->GetRivermaxManager()->ValidateLibraryIsLoaded() == false)
		{
			UE_LOGF(LogRivermax, Warning, "Can't create Rivermax Output Stream. Library isn't initialized.");
			return false;
		}

		Options = InOptions;
		Listener = &InListener;

		if (IConsoleVariable* CvarDelay = CVarRivermaxOutputRandomDelay.AsVariable())
		{
			CvarDelay->OnChangedDelegate().AddRaw(this, &FRivermaxOutStream::OnCVarRandomDelayChanged);
		}

		if (IConsoleVariable* CvarDelay = CVarRivermaxOutputTriggerTimingIssue.AsVariable())
		{
			CvarDelay->OnChangedDelegate().AddRaw(this, &FRivermaxOutStream::OnCVarTriggerTimingIssueChanged);
		}

		CacheCVarValues();
		
		// Cache API entry point
		CachedAPI = RivermaxModule->GetRivermaxManager()->GetApi();

		Stats.ChunkCompletionTracker = MakeShared<FRivermaxChunkCompletionTracker>(CachedAPI, CachedCVars.bTrackChunkCompletion);
		checkSlow(CachedAPI);

		// Enable frame boundary monitoring
		MonitoringGuid = RivermaxModule->GetRivermaxBoundaryMonitor().StartMonitoring(GetFrameRate());

		if (!InitializeStreamMemoryConfig())
		{
			return false;
		}

		// The following scope used to be initialized asynchronously; however, 
		// it caused synchronization issues between nodes. When either barrier or 
		// synchronization correction is added to nDisplay, this can be switched back 
		// to asynchronous initialization.
		{
			// Create Rivermax stream using memory configuration
			{
				// Setup audio stream settings
				rmx_output_media_stream_params OutputStreamParameters;
				CachedAPI->rmx_output_media_init(&OutputStreamParameters);
				CachedAPI->rmx_output_media_set_sdp(&OutputStreamParameters, &SDPDescription[0]);
				CachedAPI->rmx_output_media_assign_mem_blocks(&OutputStreamParameters, StreamMemory.MemoryBlocks.GetData(), StreamMemory.MemoryBlocks.Num());

				// Priority Code Point for Quality of Service. Tells the network how important this packet is. 7 is highest priority
				constexpr uint8 PCPAttribute = 7;
				CachedAPI->rmx_output_media_set_pcp(&OutputStreamParameters, PCPAttribute);

				if (StreamType == ERivermaxStreamType::ST2110_30)
				{
					//Differentiated Services Code Point. For AES67 RTP media streams, the DSCP value is set to 34.
					constexpr uint8 DSCP = 34;
					CachedAPI->rmx_output_media_set_dscp(&OutputStreamParameters, DSCP);
				}

				constexpr uint8 ECN = 0; //Explicit congestion notification in theory notifies if the packet is "congested"
				CachedAPI->rmx_output_media_set_ecn(&OutputStreamParameters, ECN);

				// Sometimes, chunk count will have more packets than needed so last ones might be 0 sized. 
				// Verify if new API work with the actual amount of packet with data or it needs the padded version 
				CachedAPI->rmx_output_media_set_packets_per_frame(&OutputStreamParameters, StreamMemory.ChunksPerFrameField * StreamMemory.PacketsPerChunk);
				CachedAPI->rmx_output_media_set_packets_per_chunk(&OutputStreamParameters, StreamMemory.PacketsPerChunk);

				// This means that this stream doesn't need dynamic header split. Refer to the description of HeaderBlockID and DataBlockID
				if (StreamMemory.DataBlockID != 0)
				{
					CachedAPI->rmx_output_media_set_stride_size(&OutputStreamParameters, StreamMemory.HeaderBlockID, StreamMemory.HeaderStrideSize);
				}
				CachedAPI->rmx_output_media_set_stride_size(&OutputStreamParameters, StreamMemory.DataBlockID, StreamMemory.PayloadSize);

				const size_t MediaBlockIndex = GetStreamIndexSDP();
				
				CachedAPI->rmx_output_media_set_idx_in_sdp(&OutputStreamParameters, MediaBlockIndex);

				rmx_stream_id NewId;
				rmx_status Status = CachedAPI->rmx_output_media_create_stream(&OutputStreamParameters, &NewId);

				if (Status == RMX_OK)
				{
					struct sockaddr_in SourceAddress;
					FMemory::Memset(&SourceAddress, 0, sizeof(SourceAddress));

					rmx_output_media_context MediaContext;
					CachedAPI->rmx_output_media_init_context(&MediaContext, NewId);

					const size_t SDPMediaIndex = MediaBlockIndex;
					CachedAPI->rmx_output_media_set_context_block(&MediaContext, SDPMediaIndex);
					Status = CachedAPI->rmx_output_media_get_local_address(&MediaContext, reinterpret_cast<sockaddr*>(&SourceAddress));
					if (Status == RMX_OK)
					{
						struct sockaddr_in DestinationAddress;
						FMemory::Memset(&DestinationAddress, 0, sizeof(DestinationAddress));

						Status = CachedAPI->rmx_output_media_get_remote_address(&MediaContext, reinterpret_cast<sockaddr*>(&DestinationAddress));
						if (Status == RMX_OK)
						{
							StreamId = NewId;

							CachedAPI->rmx_output_media_init_chunk_handle(&StreamData.ChunkHandle, StreamId);

							// This should be unique for each stream. Will be used in packet creation.
							StreamData.SynchronizationSource = FRawRTPHeader::VideoSynchronizationSource + (uint32)MediaBlockIndex;

							StreamData.FrameFieldTimeIntervalNs = 1E9 / GetFrameRate().AsDecimal();
							InitializeStreamTimingSettings();

							LogStreamDescriptionOnCreation();
							FString SDPAsAString(ANSI_TO_TCHAR(&SDPDescription[0]));
							UE_LOGF(LogRivermax, Verbose, "Created stream using SDP:\n%ls", *SDPAsAString);

							bIsActive = true;
							RivermaxThread.Reset(FRunnableThread::Create(this, TEXT("Rmax OutputStream Thread"), 128 * 1024, TPri_TimeCritical, FPlatformAffinity::GetPoolThreadMask()));
						}
						else
						{
							UE_LOGF(LogRivermax, Warning, "Failed querying destination address. Output Stream won't be created. Status: %d", Status);
						}
					}
					else
					{
						UE_LOGF(LogRivermax, Warning, "Failed querying local address. Output Stream won't be created. Status: %d", Status);
					}
				}
				else
				{
					UE_LOGF(LogRivermax, Warning, "Failed to create Rivermax output stream. Status: %d", Status);
				}
			}

			Listener->OnInitializationCompleted(bIsActive);
		}

		return true;
	}

	void FRivermaxOutStream::Uninitialize()
	{
		if (RivermaxThread != nullptr)
		{
			Stop();

			FrameAvailableSignal->Trigger();
			FrameReadyToSendSignal->Trigger();
			RivermaxThread->Kill(true);
			RivermaxThread.Reset();

			CleanupFrameManagement();

			RivermaxModule->GetRivermaxBoundaryMonitor().StopMonitoring(MonitoringGuid, GetFrameRate());
			
			UE_LOGF(LogRivermax, Log, "Rivermax Output stream has shutdown");
		}

		if (IConsoleVariable* CvarDelay = CVarRivermaxOutputRandomDelay.AsVariable())
		{
			CvarDelay->OnChangedDelegate().RemoveAll(this);
		}

		if (IConsoleVariable* CvarDelay = CVarRivermaxOutputTriggerTimingIssue.AsVariable())
		{
			CvarDelay->OnChangedDelegate().RemoveAll(this);
		}
	}

	void FRivermaxOutStream::InitializeNextFrame(const TSharedPtr<FRivermaxOutputFrame>& NextFrame)
	{
		NextFrame->LineNumber = 0;
		NextFrame->PacketCounter = 0;
		NextFrame->ChunkNumber = 0;
		NextFrame->PayloadPtr = nullptr;
		NextFrame->HeaderPtr = nullptr;
		NextFrame->FrameStartPtr = nullptr;
		NextFrame->bCaughtTimingIssue = false;
		NextFrame->Offset = 0;
	}
	
	void FRivermaxOutStream::DestroyStream()
	{
		constexpr float TimeOutSecs = 3.f;
		constexpr float TimeToWaitIncrement = 0.3f;
		float TimeWaited = 0.f;

		auto CancelUnsentChunks = [&] {
			rmx_status Status = CachedAPI->rmx_output_media_cancel_unsent_chunks(&StreamData.ChunkHandle);
			if (Status != RMX_OK)
			{
				UE_LOGF(LogRivermax, Warning, "Could not cancel unsent chunks when destroying output stream. Status: %d", Status);
			}
		};

		CancelUnsentChunks();

		rmx_status Status;
		do 
		{
			Status = CachedAPI->rmx_output_media_destroy_stream(StreamId);
			if (RMX_BUSY == Status) 
			{
				// Sometimes the chunks are not cancelled above and this could get into deadlock unless 
				// unesnt chunks are cancelled again.
				CancelUnsentChunks();
				FPlatformProcess::SleepNoStats(TimeToWaitIncrement);
				TimeWaited += TimeToWaitIncrement;
			}

		} while (Status == RMX_BUSY && TimeWaited < TimeOutSecs);

		if (Status == RMX_BUSY)
		{
			UE_LOGF(LogRivermax, Warning, "Could not destroy the stream. StreamId: %d", StreamId);
		}
	}

	bool FRivermaxOutStream::WaitForNextRound()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForNextRound);
		const uint64 CurrentTimeNanosec = RivermaxModule->GetRivermaxManager()->GetTime();
		const double CurrentPlatformTime = FPlatformTime::Seconds();
		const uint64 CurrentFrameNumber = UE::RivermaxCore::GetFrameNumber(CurrentTimeNanosec, GetFrameRate());

		switch (Options.AlignmentMode)
		{
			case ERivermaxAlignmentMode::AlignmentPoint:
			{
				CalculateNextScheduleTime_AlignementPoints(CurrentTimeNanosec, CurrentFrameNumber);
				break;
			}
			case ERivermaxAlignmentMode::FrameCreation:
			{
				CalculateNextScheduleTime_FrameCreation(CurrentTimeNanosec, CurrentFrameNumber);
				break;
			}
			default:
			{
				checkNoEntry();
			}
		}

		// Offset wakeup if desired to give more time for scheduling. 
		const uint64 WakeupTime = StreamData.NextAlignmentPointNanosec - CVarRivermaxWakeupOffset.GetValueOnAnyThread();

		uint64 WaitTimeNanosec = WakeupTime - CurrentTimeNanosec;

		// Wakeup can be smaller than current time with controllable offset
		if (WakeupTime < CurrentTimeNanosec)
		{
			WaitTimeNanosec = 0;
		}

		static constexpr float SleepThresholdSec = 5.0f / 1000;
		static constexpr float YieldTimeSec = 2.0f / 1000;
		const double WaitTimeSec = FMath::Min(WaitTimeNanosec / 1E9, 1.0);
		StreamData.LastSleepTimeNanoSec = WaitTimeNanosec;

		bool bIsFrameReady = false;
		if (StreamMemory.bUseIntermediateBuffer)
		{	
			// When using intermediate buffer, we verify if next buffer is ready sooner than wake up time
			// If a frame is ready already, we can move on. Otherwise, we wait for FrameReady signal with a
			// wait timeout. In the case of a repeated frame, we will always timeout and we won't be able to 
			// do an early copy.
			if (IsFrameAvailableToSend())
			{
				bIsFrameReady = true;
			}
			else
			{
				const uint32 WaitMs = FMath::Floor((WaitTimeSec - YieldTimeSec) * 1000.);
				do 
				{
					bIsFrameReady = FrameReadyToSendSignal->Wait(WaitMs);
				} while (!IsFrameAvailableToSend() && bIsFrameReady && bIsActive);
			}
		}
		else
		{
			// Sleep for the largest chunk of time
			if (WaitTimeSec > SleepThresholdSec)
			{
				FPlatformProcess::SleepNoStats(WaitTimeSec - YieldTimeSec);
			}
		}

		if (!bIsFrameReady)
		{
			// We are past the long sleep so no more early data access possible. Just yield until the wake up time.
			{
				// Use platform time instead of rivermax get PTP to avoid making calls to it. Haven't been profiled if it impacts
				while (FPlatformTime::Seconds() < (CurrentPlatformTime + WaitTimeSec))
				{
					FPlatformProcess::SleepNoStats(0.f);
				}
			}

			if (StreamData.bHasValidNextFrameNumber && CachedCVars.bShowOutputStats)
			{
				const uint64 AfterSleepTimeNanosec = RivermaxModule->GetRivermaxManager()->GetTime();
				const uint64 RealWaitNs = AfterSleepTimeNanosec - CurrentTimeNanosec;
				const uint64 OvershootSleep = AfterSleepTimeNanosec > StreamData.NextAlignmentPointNanosec ? AfterSleepTimeNanosec - StreamData.NextAlignmentPointNanosec : 0;
				const double OvershootSleepSec = OvershootSleep / 1e9;

				UE_LOGF(LogRivermax, Verbose, "CurrentTime %llu. OvershootSleep: %0.9f. ExpectedWait: %0.9f. RealWait: %0.9f, Scheduling at %llu. NextAlign %llu. "
					, CurrentTimeNanosec
					, OvershootSleepSec
					, (double)WaitTimeNanosec / 1E9
					, (double)RealWaitNs / 1E9
					, StreamData.NextScheduleTimeNanosec
					, StreamData.NextAlignmentPointNanosec);
			}
		}
		else
		{
			if (StreamData.bHasValidNextFrameNumber && CachedCVars.bShowOutputStats)
			{
				const uint64 AfterSleepTimeNanosec = RivermaxModule->GetRivermaxManager()->GetTime();
				UE_LOGF(LogRivermax, Verbose, "Early data available. CurrentTime %llu. Scheduling at %llu. NextAlign %llu. "
					, AfterSleepTimeNanosec
					, StreamData.NextScheduleTimeNanosec
					, StreamData.NextAlignmentPointNanosec);
			}
		}

		return bIsFrameReady;
	}

	void FRivermaxOutStream::GetNextChunk()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetNextChunk);

		bool bHasAddedTrace = false;
		rmx_status Status;

		do
		{
			Status = CachedAPI->rmx_output_media_get_next_chunk(&StreamData.ChunkHandle);
			CurrentFrame->PayloadPtr = rmx_output_media_get_chunk_strides(&StreamData.ChunkHandle, StreamMemory.DataBlockID);

			// This means that this stream doesn't need dynamic header split. Refer to the description of HeaderBlockID and DataBlockID
			if (StreamMemory.DataBlockID != 0)
			{
				CurrentFrame->HeaderPtr = rmx_output_media_get_chunk_strides(&StreamData.ChunkHandle, StreamMemory.HeaderBlockID);
			}

			if (Status == RMX_OK)
			{
				if (CurrentFrame->FrameStartPtr == nullptr)
				{
					// Stamp frame start in order to copy frame data sequentially as we query chunks
					CurrentFrame->FrameStartPtr = CurrentFrame->PayloadPtr;
				}

				break;
			}
			else if (Status == RMX_NO_FREE_CHUNK)
			{
				//We should not be here
				if (!bHasAddedTrace)
				{
					Stats.LastFrameChunkRetries++;
					UE_LOGF(LogRivermax, Verbose, "No free chunks to get for chunk '%u'. Waiting", CurrentFrame->ChunkNumber);
					TRACE_CPUPROFILER_EVENT_SCOPE(GetNextChunk::NoFreeChunk);
					bHasAddedTrace = true;
				}
			}
			else
			{
				UE_LOGF(LogRivermax, Error, "Invalid error happened while trying to get next chunks. Status: %d", Status);
				Listener->OnStreamError();
				Stop();
			}
		} while (Status != RMX_OK && bIsActive);
	}

	void FRivermaxOutStream::CommitNextChunks()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CommitNextChunks);
		rmx_status Status;
		int32 ErrorCount = 0;
		const uint64 CurrentTimeNanosec = RivermaxModule->GetRivermaxManager()->GetTime();
		uint64 ScheduleTime = CurrentFrame->ChunkNumber == 0 ? StreamData.NextScheduleTimeNanosec : 0;
		
		if (CurrentFrame->ChunkNumber == 0)
		{
			Stats.ChunkCompletionTracker->MarkChunkForTracking("First Chunk", StreamData.ChunkHandle, StreamData.NextScheduleTimeNanosec, StreamData.DeltaTimePerChunkNs, CurrentFrame->ChunkNumber);
		}

		// This is actually tracking second to last chunk. Rivermax API internally doesn't mark the last chunk as completed until after the TRoffset gap, because it is doing some other processing. 
		if (CurrentFrame->ChunkNumber == StreamMemory.ChunksPerFrameField - 2)
		{
			// CVarRivermaxOutputCommitChunksOffsetPercent allows the early start of chunk commits. The last chunk will not be completed until the next alignment point. Polling for chunk completion starts right
			// before the next frame is put on the wire which is at CVarRivermaxOutputCommitChunksOffsetPercent. Polling doesn't exit until it gets chunk completion data, which for the last chunk is right before the next
			// alignment point. Therefore, this negates the intended effect of CVarRivermaxOutputCommitChunksOffsetPercent, because the commit of the first chunk of the next frame is delayed until 
			// the last chunk of the current frame is fully completed.
			if (CVarRivermaxOutputCommitChunksOffsetPercent.GetValueOnAnyThread() < KINDA_SMALL_NUMBER)
			{
				Stats.ChunkCompletionTracker->MarkChunkForTracking("Last Chunk", StreamData.ChunkHandle, StreamData.NextScheduleTimeNanosec, StreamData.DeltaTimePerChunkNs, CurrentFrame->ChunkNumber);
			}
		}

		do
		{
			//Only first chunk gets scheduled with a timestamp. Following chunks are queued after it using 0
			if (ScheduleTime != 0)
			{
				// If scheduling time is not far away enough, force it immediately otherwise rmax_commit will throw an error 
				if (ScheduleTime <= (CurrentTimeNanosec + CachedCVars.ForceCommitImmediateTimeNanosec))
				{
					ScheduleTime = 0;
					++Stats.CommitImmediate;
				}
			}

			checkSlow(CachedAPI);

			Status = CachedAPI->rmx_output_media_commit_chunk(&StreamData.ChunkHandle, ScheduleTime);

			if (Status == RMX_OK)
			{
				break;
			}
			else if (Status == RMX_HW_SEND_QUEUE_IS_FULL)
			{
				Stats.CommitRetries++;

				// Yeild the current timeslice in case there is another equal priority thread because RMX_HW_SEND_QUEUE_IS_FULL can last a few ms.
				FPlatformProcess::SleepNoStats(0.f);
				TRACE_CPUPROFILER_EVENT_SCOPE(CommitNextChunks::QUEUEFULL);
				++ErrorCount;
			}
			else if (Status == RMX_HW_COMPLETION_ISSUE)
			{
				UE_LOGF(LogRivermax, Error, "Completion issue while trying to commit next round of chunks.");
				Listener->OnStreamError();
				Stop();
			}
			else
			{
				UE_LOGF(LogRivermax, Error, "Unhandled error (%d) while trying to commit next round of chunks.", Status);
				Listener->OnStreamError();
				Stop();
			}

		} while (Status != RMX_OK && bIsActive);

		if (bIsActive && CurrentFrame->ChunkNumber == 0 && CachedCVars.bShowOutputStats)
		{
			UE_LOGF(LogRivermax, Verbose, "Committed frame [%llu]. Scheduled for '%llu'. Aligned with '%llu'. Current time '%llu'. Was late: %d. Slack: %llu. Errorcount: %d"
				, CurrentFrame->GetFrameCounter()
				, ScheduleTime
				, StreamData.NextAlignmentPointNanosec
				, CurrentTimeNanosec
				, CurrentTimeNanosec >= StreamData.NextScheduleTimeNanosec ? 1 : 0
				, StreamData.NextScheduleTimeNanosec >= CurrentTimeNanosec ? StreamData.NextScheduleTimeNanosec - CurrentTimeNanosec : 0
				, ErrorCount);
		}
	}

	bool FRivermaxOutStream::Init()
	{
		return true;
	}

	void FRivermaxOutStream::Process_AnyThread()
	{
		using namespace UE::RivermaxCore::Private::Utils;

		// Wait for the next time a frame should be sent (based on frame interval)
		// if interm buffer is used (alignment points) and a frame is ready before frame interval
		//		Start copying data into next memory block from the intermediate buffer
		//		At frame interval:
		//			Release last sent frame if any
		//			Make next frame the one being sent
		// Otherwise
		//		FrameCreation:
		//			Release last sent frame if any
		//			Wait for a new frame to be available
		//		Alignment points:
		//			Release last sent frame if any
		// 
		// Send frame
		//		Get next chunk
		//		Continue copy to intermediate buffer if required
		//		Fill dynamic data for RTP headers of next chunk
		//		Commit next chunk
		// 
		// Restart
		{
			bool bCanEarlyCopy = false;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::Wait);
				bCanEarlyCopy = WaitForNextRound();
			}

			if (bIsActive && bCanEarlyCopy)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::PreprocessNextFrame);
				PreprocessNextFrame();
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::PrepareNextFrame);
				PrepareNextFrame();
			}

			// At this point, if there is no frame to send, move on to wait for next round
			if (CurrentFrame.IsValid() && bIsActive)
			{
				SendFrame();

				// If frame that was just sent failed timing requirements, we have to tell rivermax to skip 0 chunks 
				// in order to reset internal states. Otherwise, scheduling time / Tro isn't respected next time we schedule.
				if (CurrentFrame->bCaughtTimingIssue)
				{
					++Stats.TimingIssueCount;
					constexpr uint64 ChunksToSkip = 0;
					SkipChunks(ChunksToSkip);
				}
			}

			Stats.TotalChunkRetries += Stats.LastFrameChunkRetries;
			Stats.LastFrameChunkRetries = 0;
		}
	}

	void FRivermaxOutStream::PreprocessNextFrame()
	{
		checkSlow(Options.AlignmentMode == ERivermaxAlignmentMode::AlignmentPoint);

		TSharedPtr<FRivermaxOutputFrame> NextFrameToSend = GetNextFrameToSend();
		if (StreamType == ERivermaxStreamType::ST2110_40_TC)
		{
			check(true);
		}
		if (ensure(NextFrameToSend))
		{
			InitializeNextFrame(NextFrameToSend);

			// Now that we have the next frame, we can start copying data into it.
			// We can't get chunks since commit will only commit the chunks returned by last call to get next chunk. 
			// So, we calculate the next data and header pointer based on the current frame. 
			const uint64 CurrentRmaxTimeNanosec = RivermaxModule->GetRivermaxManager()->GetTime();
			const double CurrentPlatformTime = FPlatformTime::Seconds();
			const double TimeLeftSec = double(StreamData.NextAlignmentPointNanosec - CurrentRmaxTimeNanosec) / 1E9;
			const double TargetPlatformTimeSec = CurrentPlatformTime + TimeLeftSec;
			if (CurrentRmaxTimeNanosec < StreamData.NextAlignmentPointNanosec)
			{
				if (!StreamMemory.bUseIntermediateBuffer)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::CopyFrame);

					bool bHasDataToCopy = true;
					while (FPlatformTime::Seconds() < TargetPlatformTimeSec && bHasDataToCopy && bIsActive)
					{
						bHasDataToCopy = CopyFrameData(NextFrameToSend, reinterpret_cast<uint8*>(StreamMemory.BufferAddresses[GET_FRAME_INDEX(NextFrameToSend)]));
					}
				}

				const double PostCopyTimeLeftSec = TargetPlatformTimeSec - FPlatformTime::Seconds();
				if (PostCopyTimeLeftSec > 0)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::Waiting);
					static constexpr float YieldTimeSec = 1.0f / 1000;

					double TargetPlatformTimeToCommitSec = TargetPlatformTimeSec;

					const double CVarStartCommitEarlyPercent = FMath::Clamp(CVarRivermaxOutputCommitChunksOffsetPercent.GetValueOnAnyThread(), 0.0, 0.8);
					if (CVarStartCommitEarlyPercent > 0.)
					{
						// We would like to start comitting chunks before the alignment point, but not too early so that we don't block this thread.
						TargetPlatformTimeToCommitSec = TargetPlatformTimeSec - GetFrameRate().AsInterval() * CVarStartCommitEarlyPercent;
					}

					while (FPlatformTime::Seconds() < TargetPlatformTimeToCommitSec && bIsActive)
					{
						const double TimeLeft = TargetPlatformTimeToCommitSec - FPlatformTime::Seconds();
						const double SleepTime = TimeLeft > YieldTimeSec ? TimeLeft - YieldTimeSec : 0.0;
						FPlatformProcess::SleepNoStats(SleepTime);
					}
				}
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::WrappingUp);
				if (CurrentFrame)
				{
					constexpr bool bReleaseFrame = true;
					CompleteCurrentFrame(bReleaseFrame);
				}

				// Make the next frame to send the current one and update its state
				CurrentFrame = MoveTemp(NextFrameToSend);
			}
		}
		else
		{
			UE_LOGF(LogRivermax, Error, "Unexpected error, no frame was available.");
			Listener->OnStreamError();
			Stop();
		}
	}

	uint32 FRivermaxOutStream::Run()
	{
		if (CachedCVars.bEnableTimeCriticalThread)
		{
#if PLATFORM_WINDOWS
			::SetThreadPriority(GetCurrentThread(), THREAD_PRIORITY_TIME_CRITICAL);
#elif PLATFORM_LINUX
			sched_param SchedParam;
			SchedParam.sched_priority = sched_get_priority_max(SCHED_FIFO);
			pthread_setschedparam(pthread_self(), SCHED_FIFO, &SchedParam);
#endif // PLATFORM_WINDOWS
		}

		// Initial wait for a frame to be produced
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::InitialWait);
			FrameReadyToSendSignal->Wait();
		}

		while (bIsActive)
		{
			ShowStats();
			Process_AnyThread();
		}

		DestroyStream();

		return 0;
	}

	void FRivermaxOutStream::Stop()
	{
		bIsActive = false;
	}

	void FRivermaxOutStream::Exit()
	{

	}

	void FRivermaxOutStream::PrepareNextFrame()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareNextFrame);

		switch (Options.AlignmentMode)
		{
			case ERivermaxAlignmentMode::FrameCreation:
			{
				PrepareNextFrame_FrameCreation();
				break;
			}
			case ERivermaxAlignmentMode::AlignmentPoint:
			{
				PrepareNextFrame_AlignmentPoint();
				break;
			}
			default:
			{
				checkNoEntry();
			}
		}
	}

	void FRivermaxOutStream::PrepareNextFrame_FrameCreation()
	{
		// When aligning on frame creation, we will always wait for a frame to be available.
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::WaitForReadyFrame);
		TSharedPtr<FRivermaxOutputFrame> NextFrameToSend = GetNextFrameToSend(true);

		// In frame creation alignment, we always release the last frame sent
		if (CurrentFrame.IsValid())
		{
			constexpr bool bReleaseFrame = true;
			CompleteCurrentFrame(bReleaseFrame);
		}

		// Make the next frame to send the current one and update its state
		if (NextFrameToSend)
		{
			CurrentFrame = MoveTemp(NextFrameToSend);
			InitializeNextFrame(CurrentFrame);
		}
	}

	void FRivermaxOutStream::PrepareNextFrame_AlignmentPoint()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::GetNextFrame_AlignmentPoint);

		// When aligning on alignment points:
		// We prepare to send the next frame that is ready if there is one available
		// if none are available and bDoContinuousOutput == true
		//		Repeat the last frame
		// if none are available and bDoContinuousOutput == false
		//		Don't send a frame and go back waiting for the next alignment point
		
		TSharedPtr<FRivermaxOutputFrame> NextFrameToSend = GetNextFrameToSend();

		// If we have a new frame, release the previous one
		// If we don't have a frame and we're not doing continuous output, we release it and we won't send a new one
		// If we don't have a frame but we are doing continuous output, we will reschedule the current one, so no release.
		if (!Options.bDoContinuousOutput || NextFrameToSend)
		{
			if (CurrentFrame)
			{
				constexpr bool bReleaseFrame = true;
				CompleteCurrentFrame(bReleaseFrame);
			}

			// Make the next frame to send the current one and update its state
			if (NextFrameToSend)
			{
				CurrentFrame = MoveTemp(NextFrameToSend);
				InitializeNextFrame(CurrentFrame);
			}
		}
		else
		{
			// We finished sending a frame so complete it but don't release it as we will repeat it
			constexpr bool bReleaseFrame = false;
			CompleteCurrentFrame(bReleaseFrame);

			// We will resend the last one so just reinitialize it to resend
			InitializeNextFrame(CurrentFrame);
			
			// If intermediate buffer isn't used and frame has to be repeated, we use skip chunk method 
			// which might cause timing errors caused by chunk managements issue 
			if (!StreamMemory.bUseIntermediateBuffer)
			{
				// No frame to send, keep last one and restart its internal counters
				UE_LOGF(LogRivermax, Verbose, "No frame to send. Reusing last frame with Frame Counter: %llu", CurrentFrame->GetFrameCounter());

				// Since we want to resend last frame, we need to fast forward chunk pointer to re-point to the one we just sent
				SkipChunks(StreamMemory.ChunksPerFrameField * (Options.NumberOfBuffers - 1));
			}
		}
	}

	void FRivermaxOutStream::CleanupFrameManagement()
	{
	}

	void FRivermaxOutStream::InitializeStreamTimingSettings()
	{
		using namespace UE::RivermaxCore::Private::Utils;

		const double TROOverride = CVarRivermaxOutputTROOverride.GetValueOnAnyThread();
		if (TROOverride != 0)
		{
			TransmitOffsetNanosec = TROOverride * 1E9;
		}
		else
		{
			double FrameIntervalNs = StreamData.FrameFieldTimeIntervalNs;
			const bool bIsProgressive = true;//todo MediaConfiguration.IsProgressive() 
			uint32 PacketsInFrameField = StreamMemory.PacketsPerFrame;
			if (bIsProgressive == false)
			{
				FrameIntervalNs *= 2;
				PacketsInFrameField *= 2;
			}

			auto ComputeVideoTROMultiplier = [](bool bProgressive, int32 AlignedHeight) -> double
			{
				// See https://ieeexplore.ieee.org/stamp/stamp.jsp?tp=&arnumber=8165971 for reference
				// Gapped PRS doesn't support non standard resolution.
				if (bProgressive)
				{
					// As defined by SMPTE 2110-21 6.3.2
					return AlignedHeight >= FullHDHeight ? (43.0 / 1125.0) : (28.0 / 750.0);
				}
				else
				{
					// As defined by SMPTE 2110-21 6.3.3
					if (AlignedHeight >= FullHDHeight) { return (22.0 / 1125.0); }
					if (AlignedHeight >= 576)          { return (26.0 / 625.0);  }
					return (20.0 / 525.0);
				}
			};

			if (StreamType == ERivermaxStreamType::ST2110_20)
			{
				TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(StreamType);
				TransmitOffsetNanosec = (uint64)(ComputeVideoTROMultiplier(bIsProgressive, VideoOptions->AlignedResolution.Y) * FrameIntervalNs);
			}
			else if (StreamType == ERivermaxStreamType::ST2110_40 || StreamType == ERivermaxStreamType::ST2110_40_TC)
			{
				// ST2110-40 ANC TRoffset matches the companion video stream (SMPTE ST 2110-40 Sec. 7).
				TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(ERivermaxStreamType::ST2110_20);
				if (VideoOptions.IsValid())
				{
					TransmitOffsetNanosec = (uint64)(ComputeVideoTROMultiplier(bIsProgressive, VideoOptions->AlignedResolution.Y) * FrameIntervalNs);
				}
			}
			// TODO: Add TRoffset for ST2110_30 audio.
		}

		StreamData.DeltaTimePerChunkNs = (static_cast<uint64>(StreamData.FrameFieldTimeIntervalNs) - TransmitOffsetNanosec) / StreamMemory.ChunksPerFrameField;
	}

	const FFrameRate& FRivermaxOutStream::GetFrameRate() const
	{
		return Options.StreamOptions[StreamType]->FrameRate;
	}

	const FString& FRivermaxOutStream::GetStreamAddress() const
	{
		return Options.StreamOptions[StreamType]->StreamAddress;
	}

	const FString& FRivermaxOutStream::GetInterfaceAddress() const
	{
		return Options.StreamOptions[StreamType]->InterfaceAddress;
	}

	uint32 FRivermaxOutStream::GetPort() const
	{
		return Options.StreamOptions[StreamType]->Port;
	}

	uint64 FRivermaxOutStream::GetStreamIndexSDP() const
	{
		return Options.StreamOptions[StreamType]->StreamIndex;
	}

	void FRivermaxOutStream::LogStreamDescriptionOnCreation() const
	{
		TStringBuilder<512> StreamDescription;

		StreamDescription.Appendf(TEXT("Output stream started sending on stream %s:%d using interface %s. ")
			, *GetStreamAddress()
			, GetPort()
			, *GetInterfaceAddress());

		// Matches ERivermaxStreamType
		const TArray<FString> StreamTypeToStrMap =
		{
			"Video",
			"Audio",
			"Ancillary",
			"Ancillary_TC"
		};

		check(StreamTypeToStrMap.Num() == ERivermaxStreamType::MAX);

		StreamDescription.Appendf(TEXT("StreamType = %s "), *StreamTypeToStrMap[StreamType]);
		UE_LOGF(LogRivermax, Display, "%ls", *FString(StreamDescription));
	}


	void FRivermaxOutStream::ShowStats()
	{
		if (CachedCVars.bShowOutputStats)
		{
			const double CurrentTime = FPlatformTime::Seconds();
			if (CurrentTime - LastStatsShownTimestamp > CachedCVars.ShowOutputStatsIntervalSeconds)
			{
				LastStatsShownTimestamp = CurrentTime;
				UE_LOGF(LogRivermax, Log, "Stats: FrameSent: %llu. CommitImmediate: %u. CommitRetries: %u. ChunkRetries: %u. ChunkSkippingRetries: %u. Timing issues: %llu", Stats.FramesSentCounter, Stats.CommitImmediate, Stats.CommitRetries, Stats.TotalChunkRetries, Stats.ChunkSkippingRetries, Stats.TimingIssueCount);
			}
		}
	}

	bool FRivermaxOutStream::IsGPUDirectSupported() const
	{
		return bUseGPUDirect;
	}

	void FRivermaxOutStream::CalculateNextScheduleTime_AlignementPoints(uint64 CurrentClockTimeNanosec, uint64 CurrentFrameNumber)
	{
		// Frame number we will want to align with
		uint64 NextFrameNumber = CurrentFrameNumber;

		bool bFoundValidTimings = true;
		
		if (StreamData.bHasValidNextFrameNumber == false)
		{
			// Now that the stream starts when a frame was produced, we can reduce our wait
			// We wait one frame here to start sending at the next frame boundary.
			// Since it takes a frame to send it, we could detect if we are in the first 10% (arbitrary)
			// of the interval and start sending right away but we might be overlapping with the next one
			NextFrameNumber = CurrentFrameNumber + 1;
		}
		else
		{	
			// Case where we are back and frame number is the previous one. Depending on offsets, this could happen
			if (CurrentFrameNumber == StreamData.NextAlignmentPointFrameNumber - 1)
			{
				NextFrameNumber = StreamData.NextAlignmentPointFrameNumber + 1;
				UE_LOGF(LogRivermax, Verbose, "Scheduling last frame was faster than expected. (CurrentFrame: '%llu' LastScheduled: '%llu') Scheduling for following expected one."
					, CurrentFrameNumber
					, StreamData.NextAlignmentPointFrameNumber);
			}
			else
			{
				// We expect current frame number to be the one we scheduled for the last time or greater if something happened
				if (CurrentFrameNumber >= StreamData.NextAlignmentPointFrameNumber)
				{
					// Verify if last frame had timing issues. If yes, we skip next interval
					if (CurrentFrame && CurrentFrame->bCaughtTimingIssue)
					{
						NextFrameNumber = StreamData.NextAlignmentPointFrameNumber + 2;
						UE_LOG(LogRivermax, Warning, TEXT("Timing issue detected during frame %llu. Skipping frame %llu to keep sync."), StreamData.NextAlignmentPointFrameNumber, StreamData.NextAlignmentPointFrameNumber + 1);
					}
					else
					{
						// If current frame is greater than last scheduled, we missed an alignment point.
						const uint64 DeltaFrames = CurrentFrameNumber - StreamData.NextAlignmentPointFrameNumber;
						if (DeltaFrames >= 1)
						{
							UE_LOGF(LogRivermax, Warning, "Output missed %llu frames.", DeltaFrames);

							// If we missed a sync point, this means that last scheduled frame might still be ongoing and 
							// sending it might be crossing the frame boundary so we skip one entire frame to empty the queue.
							NextFrameNumber = CurrentFrameNumber + 2;
						}
						else
						{
							NextFrameNumber = CurrentFrameNumber + 1;
						}
					}
				}
				else
				{
					// This is not expected (going back in time) but we should be able to continue. Scheduling immediately
					ensureMsgf(false, TEXT("Unexpected behaviour during output stream's alignment point calculation. Current time has gone back in time compared to last scheduling."));
					bFoundValidTimings = false;
				}
			}
		}

		// Get next alignment point based on the frame number we are aligning with
		const uint64 NextAlignmentNano = UE::RivermaxCore::GetAlignmentPointFromFrameNumber(NextFrameNumber, GetFrameRate());

		// Add Tro offset to next alignment point and configurable offset
		StreamData.NextAlignmentPointNanosec = NextAlignmentNano;
		StreamData.NextScheduleTimeNanosec = NextAlignmentNano + TransmitOffsetNanosec + CVarRivermaxScheduleOffset.GetValueOnAnyThread();
		StreamData.LastAlignmentPointFrameNumber = StreamData.NextAlignmentPointFrameNumber;
		StreamData.NextAlignmentPointFrameNumber = NextFrameNumber;

		StreamData.bHasValidNextFrameNumber = bFoundValidTimings;
	}

	void FRivermaxOutStream::CalculateNextScheduleTime_FrameCreation(uint64 CurrentClockTimeNanosec, uint64 CurrentFrameNumber)
	{
		double NextWaitTime = 0.0;
		if (StreamData.bHasValidNextFrameNumber == false)
		{
			StreamData.NextAlignmentPointNanosec = CurrentClockTimeNanosec;
			StreamData.NextScheduleTimeNanosec = StreamData.NextAlignmentPointNanosec + CVarRivermaxScheduleOffset.GetValueOnAnyThread();
			StreamData.NextAlignmentPointFrameNumber = CurrentFrameNumber;
			StreamData.bHasValidNextFrameNumber = true;
		}
		else
		{
			// In this mode, we just take last time we started to send and add a frame interval
			StreamData.NextAlignmentPointNanosec = StreamData.LastSendStartTimeNanoSec + StreamData.FrameFieldTimeIntervalNs;
			StreamData.NextScheduleTimeNanosec = StreamData.NextAlignmentPointNanosec + CVarRivermaxScheduleOffset.GetValueOnAnyThread();
			StreamData.NextAlignmentPointFrameNumber = UE::RivermaxCore::GetFrameNumber(StreamData.NextAlignmentPointNanosec, GetFrameRate());
		}
	}

	bool FRivermaxOutStream::ReserveFrame(uint64 FrameCounter) const
	{
		return false;
	}

	void FRivermaxOutStream::OnFrameReadyToBeSent()
	{	
		FrameReadyToSendSignal->Trigger();
	}

	void FRivermaxOutStream::OnFrameReadyToBeUsed()
	{
		FrameAvailableSignal->Trigger();
	}

	void FRivermaxOutStream::OnPreFrameReadyToBeSent()
	{
		Listener->OnPreFrameEnqueue();
	}

	void FRivermaxOutStream::OnFrameManagerCriticalError()
	{
		Listener->OnStreamError();
		Stop();
	}

	void FRivermaxOutStream::CacheCVarValues()
	{
		CachedCVars.bEnableCommitTimeProtection = CVarRivermaxOutputEnableTimingProtection.GetValueOnAnyThread() != 0;
		CachedCVars.ForceCommitImmediateTimeNanosec = CVarRivermaxOutputForceImmediateSchedulingThreshold.GetValueOnAnyThread();
		CachedCVars.SkipSchedulingTimeNanosec = CVarRivermaxOutputSkipSchedulingCutOffTime.GetValueOnAnyThread() * 1E3;
		CachedCVars.bUseSingleMemblock = CVarRivermaxOutputUseSingleMemblock.GetValueOnAnyThread() == 1;
		CachedCVars.bEnableTimeCriticalThread = CVarRivermaxOutputEnableTimeCriticalThread.GetValueOnAnyThread() != 0;
		CachedCVars.bShowOutputStats = CVarRivermaxOutputShowStats.GetValueOnAnyThread() != 0;
		CachedCVars.ShowOutputStatsIntervalSeconds = CVarRivermaxOutputShowStatsInterval.GetValueOnAnyThread();
		CachedCVars.bPrefillRTPHeaders = CVarRivermaxOutputPrefillRTPHeaders.GetValueOnAnyThread();
		CachedCVars.bTrackChunkCompletion = CVarRivermaxOutputTrackChunkCompletion.GetValueOnAnyThread();
	}

	bool FRivermaxOutStream::IsChunkOnTime() const
	{
		switch (Options.AlignmentMode)
		{
		case ERivermaxAlignmentMode::AlignmentPoint:
		{
			return IsChunkOnTime_AlignmentPoints();
		}
		case ERivermaxAlignmentMode::FrameCreation:
		{
			return IsChunkOnTime_FrameCreation();
		}
		default:
		{
			checkNoEntry();
			return false;
		}
		}
	}

	bool FRivermaxOutStream::IsChunkOnTime_FrameCreation() const
	{
		return true;
	}

	bool FRivermaxOutStream::IsChunkOnTime_AlignmentPoints() const
	{
		if (CachedCVars.bEnableCommitTimeProtection)
		{
			// Calculate at what time this chunk is supposed to be sent
			const uint64 NextChunkCommitTime = StreamData.NextScheduleTimeNanosec + (CurrentFrame->ChunkNumber * StreamData.DeltaTimePerChunkNs);

			// Verify if we are on time to send it. Use CVar to tighten / extend needed window. 
			// This is to avoid messing up timing
			const uint64 CurrentTime = RivermaxModule->GetRivermaxManager()->GetTime();
			if (NextChunkCommitTime <= (CurrentTime + CachedCVars.SkipSchedulingTimeNanosec))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RmaxOut::ChunkTooLate);
				return false;
			}

			// Add other causes of timing issues. 
			// Possible options : Chunk warnings, Last commit time too close to frame boundary, etc...
		}
		
		return true;
	}

	void FRivermaxOutStream::OnMemoryChunksCopied(const TSharedPtr<FBaseDataCopySideCar>& Sidecar)
	{
	}

	void FRivermaxOutStream::OnCVarRandomDelayChanged(IConsoleVariable* Var)
	{
		bTriggerRandomDelay = true;
	}

	void FRivermaxOutStream::OnCVarTriggerTimingIssueChanged(IConsoleVariable* Var)
	{
		ScheduledTimeForArtificialTimingIssue = 0.0;
		ETimingIssueMode State = static_cast<ETimingIssueMode>(Var->GetInt());
		if (State == ETimingIssueMode::Delay)
		{
			ScheduledTimeForArtificialTimingIssue = FPlatformTime::Seconds() + 10.;
		}
		else if (State == ETimingIssueMode::Immediate)
		{
			ScheduledTimeForArtificialTimingIssue = FPlatformTime::Seconds();
		}
	}

	void FRivermaxOutStream::CalculateFrameTimestamp()
	{
		using namespace UE::RivermaxCore::Private::Utils;

		// For now, in order to be able to use a framelocked input, we pipe frame number in the timestamp for a UE-UE interaction
		// Follow up work to investigate adding this in RTP header
		uint64 InputTime = StreamData.NextAlignmentPointNanosec;
		if (Options.bDoFrameCounterTimestamping)
		{
			InputTime = UE::RivermaxCore::GetAlignmentPointFromFrameNumber(CurrentFrame->GetFrameCounter(), GetFrameRate());
		}

		CurrentFrame->MediaTimestamp = GetTimestampFromTime(InputTime, MediaClockSampleRate);

		//TODO: Use engine timecode. Aggregated in thread safe place.
		CurrentFrame->Timecode = GetTimecodeFromTime(InputTime, MediaClockSampleRate, GetFrameRate());
	}

	void FRivermaxOutStream::SkipChunks(uint64 ChunkCount)
	{
		bool bHasAddedTrace = false;
		rmx_status Status;
		do
		{
			checkSlow(CachedAPI);
			Status = CachedAPI->rmx_output_media_skip_chunks(&StreamData.ChunkHandle, ChunkCount);
			if (Status != RMX_OK)
			{
				if (Status == RMX_NO_FREE_CHUNK)
				{
					// Wait until there are enough free chunk to be skipped
					if (!bHasAddedTrace)
					{
						UE_LOGF(LogRivermax, Warning, "No chunks ready to skip. Waiting");
						TRACE_CPUPROFILER_EVENT_SCOPE(NoFreeChunk);
						bHasAddedTrace = true;
					}
				}
				else
				{
					ensure(false);
					UE_LOGF(LogRivermax, Error, "Invalid error happened while trying to skip chunks. Status: %d.", Status);
					Listener->OnStreamError();
					Stop();
				}
			}
		} while (Status != RMX_OK && bIsActive);
	}

	void FRivermaxOutStream::SendFrame()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SendFrame);
		StreamData.LastSendStartTimeNanoSec = RivermaxModule->GetRivermaxManager()->GetTime();

		if (bTriggerRandomDelay)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutStream::RandomDelay);
			bTriggerRandomDelay = false;
			FPlatformProcess::SleepNoStats(CVarRivermaxOutputRandomDelay.GetValueOnRenderThread()/1000.);
		}

		// Calculate frame's timestamp only once and reuse in RTP build
		CalculateFrameTimestamp();

		const uint32 MediaFrameNumber = Utils::TimestampToFrameNumber(CurrentFrame->MediaTimestamp, GetFrameRate());
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutSendingFrameTraceEvents[MediaFrameNumber % 10]);

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FRivermaxTracingUtils::RmaxOutMediaCapturePipeTraceEvents[CurrentFrame->GetFrameCounter() % 10]);
		UE_LOGF(LogRivermax, VeryVerbose, "RmaxRX Sending frame number %u with timestamp %u.", MediaFrameNumber, CurrentFrame->MediaTimestamp);

		// Process completions of the previous frame's chunks.
		Stats.ChunkCompletionTracker->PollAndReportCompletionOfTrackedChunks();

		ETimingIssueMode TriggerTimingIssue = static_cast<ETimingIssueMode>(CVarRivermaxOutputTriggerTimingIssue.GetValueOnAnyThread());

		do
		{
			if (bIsActive)
			{
				GetNextChunk();
			}

			if (bIsActive && StreamMemory.bUseIntermediateBuffer && ((CurrentFrame->ChunkNumber % StreamMemory.ChunkSpacingBetweenMemcopies) == 0))
			{
				CopyFrameData(CurrentFrame, reinterpret_cast<uint8*>(CurrentFrame->FrameStartPtr));
			}

			if (bIsActive)
			{
				SetupRTPHeadersForChunk();
			}

			if (bIsActive)
			{
				if (!CurrentFrame->bCaughtTimingIssue)
				{
					// As long as our frame is good, verify if we commit chunks before it is expected to be sent
					// We keep committing the frame even if we detect timing issue to avoid having to skip
					// chunks in the internals of Rivermax and keep it going for entirety of frames. 
					// We skip an interval instead but it is quite drastic.
					const bool bIsChunkOnTime = IsChunkOnTime();
					CurrentFrame->bCaughtTimingIssue = !bIsChunkOnTime;

					if ((TriggerTimingIssue == ETimingIssueMode::Immediate ||
						TriggerTimingIssue == ETimingIssueMode::Delay)
						&& (FPlatformTime::Seconds() > ScheduledTimeForArtificialTimingIssue && ScheduledTimeForArtificialTimingIssue > 0.0))
					{
						CurrentFrame->bCaughtTimingIssue = true;
						ScheduledTimeForArtificialTimingIssue = 0.0;
					}
					else if (TriggerTimingIssue == ETimingIssueMode::RandomContinuous && FMath::FRandRange(0., 1.) < .1)
					{
						CurrentFrame->bCaughtTimingIssue = true;
					}
				}

				CommitNextChunks();
			}

			//Update frame progress
			if (bIsActive)
			{
				Stats.TotalPacketSent += StreamMemory.PacketsPerChunk;
				++CurrentFrame->ChunkNumber;
			}
		} while (CurrentFrame->ChunkNumber < StreamMemory.ChunksPerFrameField && bIsActive);

		Stats.FramesSentCounter++;
	}

	void FRivermaxOutStream::GetLastPresentedFrame(FPresentedFrameInfo& OutFrameInfo) const
	{
		FScopeLock Lock(&PresentedFrameCS);
		OutFrameInfo = LastPresentedFrame;
	}

	void FRivermaxOutStream::CompleteCurrentFrame(bool bReleaseFrame)
	{
		if (ensure(CurrentFrame))
		{
			{
				FScopeLock Lock(&PresentedFrameCS);
				LastPresentedFrame.PresentedFrameBoundaryNumber = StreamData.LastAlignmentPointFrameNumber;
				LastPresentedFrame.RenderedFrameNumber = CurrentFrame->GetFrameCounter();
			}

			// We don't release when there is no new frame, so we keep a hold on it to repeat it.
			if (bReleaseFrame)
			{
				CurrentFrame.Reset();
			}
		}
	}

	void FRivermaxChunkCompletionTracker::MarkChunkForTracking(const FString& ChunkId, const rmx_output_media_chunk_handle& ChunkMarkedForCompletion, uint64 FrameScheduledTimeNanosec, uint64 DeltaTimePerChunkNs, uint32 ChunkNumber)
	{
		if (!bIsActive)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxChunkCompletionTracker::MarkChunkForTracking);

		// ExpectedChunkCompletionTime is more of a prediction and isn't precise. For that reason adding a margin.
		constexpr double ChunkCompletionMargin = 1.1; // 10% Margin;
		const uint64 ExpectedChunkCompletionTime = FrameScheduledTimeNanosec + ((ChunkNumber + 1) * DeltaTimePerChunkNs) + static_cast<uint64>(static_cast<double>(DeltaTimePerChunkNs) * ChunkCompletionMargin);

		FTrackedChunkInfo TrackedChunk;
		TrackedChunk.GeneratedToken = GetTypeHash(ChunkId);
		TrackedChunk.HumanReadableString = ChunkId;
		TrackedChunk.ChunkHandle = ChunkMarkedForCompletion;
		TrackedChunk.FrameScheduledTimeNs = FrameScheduledTimeNanosec;
		TrackedChunk.ExpectedCompletionTime = ExpectedChunkCompletionTime;

		rmx_status Status = CachedAPI->rmx_output_media_mark_chunk_for_tracking(&TrackedChunk.ChunkHandle, TrackedChunk.GeneratedToken);

		check(Status == RMX_OK);

		// Chunks are marked for tracking at the end of the frame. At the beginning frame all chunks should be polled and cleared.
		check(!TrackedChunksIds.Contains(ChunkId));
		TrackedChunksIds.Add(ChunkId);
		TrackedChunks.Add(MoveTemp(TrackedChunk));
	}

	void FRivermaxChunkCompletionTracker::PollAndReportCompletionOfTrackedChunks()
	{
		if (!bIsActive || TrackedChunks.Num() == 0)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxChunkCompletionTracker::PollAndReportCompletionOfTrackedChunks);

		rmx_status Status = RMX_OK;

		for (const FTrackedChunkInfo& TrackedChunk : TrackedChunks)
		{
			const FString& HumanReadableString = TrackedChunk.HumanReadableString;
			const uint64& Token = TrackedChunk.GeneratedToken;
			const rmx_output_media_chunk_handle& ChunkHandle = TrackedChunk.ChunkHandle;
			const uint64& FrameScheduledTimeNs = TrackedChunk.FrameScheduledTimeNs;

			uint64 TimeStamp = 0, RecordedToken = 0;

			do
			{
				Status = CachedAPI->rmx_output_media_poll_for_completion(&ChunkHandle);
				FPlatformProcess::SleepNoStats(0.f);
			} while (Status != RMX_OK);

			const rmx_output_chunk_completion* ChunkCompletion = CachedAPI->rmx_output_media_get_last_completion(&ChunkHandle);
			if (!ChunkCompletion)
			{
				UE_LOGF(LogRivermax, Error, "ChunkCompletion is invalid. Rivermax stream is either shut down or chunk handle is invalid.");
				continue;
			}
			RecordedToken = ((const rmx_output_chunk_completion_metadata*)(const void*)ChunkCompletion)->user_token;
			TimeStamp = ((const rmx_output_chunk_completion_metadata*)(const void*)ChunkCompletion)->timestamp;

			{
				// If the token of the last completed chunk isn't the same as the one expected by the iterator, it means that the last 
				// transferred chunk was out of order.
				checkf(Token == RecordedToken, TEXT("Chunk was transferred out of order."));

				// Similar to the above if the completed chunk's timestamp is behind the previous chunk's timestamp
				checkf(LastTimeStamp < TimeStamp, TEXT("Chunk was transferred out of order."));
			}

			// Signed int since we get into situation where chunks are completed before frame scheduled time. In Microseconds.
			int64 TimeDeltaAlignmentPoint = (static_cast<int64>(TimeStamp) - static_cast<int64>(FrameScheduledTimeNs))/1000;
			int64 TimeDeltaExpectedChunkCompletionTime = (static_cast<int64>(TimeStamp) - static_cast<int64>(TrackedChunk.ExpectedCompletionTime)) / 1000;

			UE_LOGF(LogRivermax, VeryVerbose, "Chunk \"%ls\" was completed with timestamp: %llu nanoseconds, which is %lld microseconds away from the expected alignment point.\n"
				"The chunk is %lld microseconds away from the predicted chunk completion time."
				, *HumanReadableString, TimeStamp, TimeDeltaAlignmentPoint, TimeDeltaExpectedChunkCompletionTime);

			if (TimeDeltaAlignmentPoint < 0)
			{
				UE_LOGF(LogRivermax, Error, "Chunk \"%ls\" was completed %lld microseconds before the first packet was expected to be on the wire. ", *HumanReadableString, FMath::Abs(TimeDeltaAlignmentPoint));
			}

			// ExpectedCompletionTime is more of a prediction and isn't precise. It also depends on Rivermax internal workings. Therefore not being on time in this case is not an error.
			if (TimeStamp > TrackedChunk.ExpectedCompletionTime)
			{
				UE_LOGF(LogRivermax, Verbose, "Chunk \"%ls\" was completed %lld microseconds after it was expected to be completed. ", *HumanReadableString, FMath::Abs(TimeDeltaExpectedChunkCompletionTime));
			}

			LastTimeStamp = TimeStamp;
		}

		TrackedChunks.Empty();
		TrackedChunksIds.Empty();
	}
}

