// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRivermaxOutputStream.h"

#include "Async/Future.h"
#include "Containers/SpscQueue.h"
#include "HAL/Runnable.h"
#include "RivermaxWrapper.h"
#include "RivermaxOutputFrame.h"
#include "RivermaxTypes.h"
#include "RTPHeader.h"


class FEvent;
class IRivermaxCoreModule;

namespace UE::RivermaxCore::Private
{
	class FFrameManager;
	class FBaseFrameAllocator;
	struct FBaseDataCopySideCar;

	using UE::RivermaxCore::FRivermaxOutputOptions;
	using UE::RivermaxCore::FRivermaxOutputInfoVideo;


	/** Struct holding configuration information with regards to stream memory and packetization */
	struct FRivermaxOutputStreamMemory
	{
		/** 
		* Size of each data payload packet that will be used. In case of single SRD this payload 
		* size will be aligned to equally divisible parts of image line width.
		*/
		uint16 PayloadSize = 0;

		/** Number of pixel group per packet */
		uint32 PixelGroupPerPacket = 0;

		/** Number of pixels per packet */
		uint32 PixelsPerPacket = 0;

		/** Number of pixels per frame */
		uint32 PixelsPerFrame = 0;

		/** Stride of RTP header data. */
		uint32 HeaderStrideSize = 20;

		/** Number of lines packed inside a chunk. Can be controlled with cvar */
		uint32 LinesInChunk = 4;

		/** Number of packets per line.  */
		uint32 PacketsInLine = 0;

		/** Number of packets per chunk. Depends on LinesInChunk */
		uint32 PacketsPerChunk = 0;

		/** Number of frames per memory block. */
		uint32 FramesFieldPerMemoryBlock = 0;

		/** Number of packets per frame */
		uint32 PacketsPerFrame = 0;

		/** Number of packets per memory block */
		uint32 PacketsPerMemoryBlock = 0;

		/**Number of chunks per frame  */
		uint32 ChunksPerFrameField = 0;

		/** Number of chunks per memory block */
		uint32 ChunksPerMemoryBlock = 0;

		/** Number of memory block */
		uint32 MemoryBlockCount = 0; 
		
		/** Whether intermediate buffer is used and captured frame has to be copied over again. */
		bool bUseIntermediateBuffer = false;

		/** Number of slices we split frame data into when copying it into intermediate buffer */
		uint32 FrameMemorySliceCount = 1;

		/** Chunk committed between each memcopy of frame data. Helps respect timing. */
		uint32 ChunkSpacingBetweenMemcopies = 1;

		/** Memory blocks allocated by Rivermax which is where packet data is stored. */
		TArray<rmx_output_media_mem_block> MemoryBlocks;

		/** 
		* Data Sub block ID. When Number of sub blocks is more than one (which is the case if dynamic header split is used), 
		* the first sub block is reserved for headers as per API docs. 
		*/
		uint8 HeaderBlockID = 0;

		/** Data Sub block ID. For ANC we don't need headers to be split, so DataBlockID will be set 0. */
		uint8 DataBlockID = 1;

		/** Array with each packet size */
		TArray<uint16_t> PayloadSizes; 

		/** Array with each RTP header size */
		TArray<uint16_t> HeaderSizes;

		/** 
		* Contains raw memory intialized to contain RTP headers for each packet per memory block. 
		* For example for Video this will store an array of FVideoRTPHeader's for each packet per memory block. 
		* The exact stride for each header is represented by HeaderStrideSize.
		*/
		TArray<TSharedPtr<uint8>> RTPHeaders;

		/** Start addresses of each buffer in memblock */
		TArray<void*> BufferAddresses;
	};

	/** 
	* A helper class keeping track of chunks marked for completion tracking. 
	*/
	class FRivermaxChunkCompletionTracker
	{
	public:
		FRivermaxChunkCompletionTracker(const RIVERMAX_API_FUNCTION_LIST* InCachedAPI, bool bInIsActive)
			: bIsActive(bInIsActive)
			, CachedAPI(InCachedAPI)
		{
		}

		/**
		* Adds a chunk to the list of tracked chunks. 
		* ChunkId is a string used for human identification.
		* ChunkMarkedForCompletion is Rivermax internal handle.
		* NextScheduleTimeNanosec is the time when the first packet of the frame is supposed to be on the wire.
		* DeltaTimePerChunk is predicted inter-chunk completion time based on frame rate and number of chunks (FrameRate/NumberOfChunks). In Nanosecs
		* ChunkNumber is the sequential number of the chunk in the frame.
		* 
		* Notes: If you mark chunk for tracking keep in mind that last chunk is completed at TRoffset from the next alignment point due to how it is setup in Rivermax. 
		* In ideal case scenario the last chunk will complete when the next alignment point starts, so if CVarRivermaxOutputCommitChunksOffsetPercent is used 
		* the chunk completion polling will eliminate the commit offset by waiting until the next alignment point.
		*/
		void MarkChunkForTracking(const FString& ChunkId, const rmx_output_media_chunk_handle& ChunkMarkedForCompletion, uint64 NextScheduleTimeNanosec, uint64 DeltaTimePerChunkNs, uint32 ChunkNumber);
		
		/** 
		* Processes all chunks that are marked for completion and outputs.
		* Should be called at the end of the frame.
		* Logs the time when chunks were completed (put on the wire) and delta between when the 
		* first packet is supposed to be on the wire and all packets in the first chunk were actually on the wire.
		*/
		void PollAndReportCompletionOfTrackedChunks();

	private:

		/** Struct containing all the required information to track the chunk and make sure it is completed on time. */
		struct FTrackedChunkInfo
		{
			/** String that is used to create a unique token for tracking in rivermax API. Also used for logging. Must be unique. */
			FString HumanReadableString;

			/** Token used for tracking the chunk. Generated from the provided string. */
			uint64 GeneratedToken;

			/** Chunk handle to keep track of. */
			rmx_output_media_chunk_handle ChunkHandle;

			/**
			* The scheduled time for the first packet of the frame (Alignment point + TRoffset + user specified offset if set).
			* In Nanoseconds.
			*/
			uint64 FrameScheduledTimeNs;

			/**
			* Time in nanoseconds when this chunk is expected to be completed. This measurement based on calculated interchunk timing 
			* and isn't precise, therefore has a small margin added to it.
			*/
			uint64 ExpectedCompletionTime;
		};

		/** Timestamp of the last polled chunk. Used for making sure that chunks are completed in order. */
		uint64 LastTimeStamp = 0;

		/** Is chunk tracking active. */
		bool bIsActive = false;

		/**
		* Chunk handle information to keep track of.
		*/ 
		TArray<FTrackedChunkInfo> TrackedChunks;

		/**
		* This set is for fast look up if the chunk is already in the list based on the string ID provided when chunk is added for tracking.
		*/
		TSet<FString> TrackedChunksIds;

		const RIVERMAX_API_FUNCTION_LIST* CachedAPI;
	};

	struct FRivermaxOutputStreamStats
	{
		/** Chunk retries that were required since stream was started */
		uint32 TotalChunkRetries = 0;
		
		/** Chunk retries that happened during last frame */
		uint32 LastFrameChunkRetries = 0;
		
		/** Chunk skipping retries that happened since stream was started */
		uint32 ChunkSkippingRetries = 0;
		
		/** Total packets that have been sent since stream was started */
		uint32 TotalPacketSent = 0;
		
		/** Number of retries that were required when committing and queue was full since stream was started */
		uint32 CommitRetries = 0;

		/** Immediate commits that were done because we got there too close to scheduling time */
		uint32 CommitImmediate = 0;

		/** Number of frames that were sent since stream was started */
		uint64 FramesSentCounter = 0;

		/** Frames that had timing issues since stream was started */
		uint64 TimingIssueCount = 0;

		/** A helper that manages completion tracking of Rivermax chunks. Enabled with a CVarRivermaxOutputTrackChunkCompletion. */
		TSharedPtr<FRivermaxChunkCompletionTracker> ChunkCompletionTracker = nullptr;
	};
	
	struct FRivermaxOutputStreamData
	{
		// Handle used to retrieve chunks associated with output stream
		rmx_output_media_chunk_handle ChunkHandle;

		/** Current sequence number being done */
		uint32 SequenceNumber = 0;

		/**  The SynchronizationSource (SSRC) field in the RTP header is a 32-bit identifier used to group RTP packets into a single stream. */
		uint32 SynchronizationSource = FRawRTPHeader::VideoSynchronizationSource;

		/** Time interval between frames in nanoseconds. */
		double FrameFieldTimeIntervalNs = 0.0;

		/** Next alignment point based on PTP standard */
		uint64 NextAlignmentPointNanosec = 0;

		/** Next schedule time using 2110 gapped model timing and controllable offset */
		uint64 NextScheduleTimeNanosec = 0;

		/** Whether next alignment frame number is deemed valid or not to detect missed frames. */
		bool bHasValidNextFrameNumber = false;
		
		/** Next alignment point frame number treated to detect missed frames */
		uint64 NextAlignmentPointFrameNumber = 0;

		/** Last alignment point frame number we have processed*/
		uint64 LastAlignmentPointFrameNumber = 0;

		/** Timestamp at which we started commiting a frame */
		uint64 LastSendStartTimeNanoSec = 0;
		
		/** Keeping track of how much time was slept last round. */
		uint64 LastSleepTimeNanoSec = 0;

		/** How long is it expected to transmit all packets in the chunk. */
		uint64 DeltaTimePerChunkNs = 0;
	};

	/** Struct holding various cached cvar values that can't be changed once stream has been created and to avoid calling anythread getters continuously */
	struct FOutputStreamCachedCVars
	{
		/** Whether timing protection is active and next frame interval is skipped if it happens */
		bool bEnableCommitTimeProtection = true;

		/** Time padding from scheduling time required to avoid skipping it */
		uint64 SkipSchedulingTimeNanosec = 0;

		/** 
		 * Time from scheduling required to not commit it immediately 
		 * Rivermax sdk will throw an error if time is in the past when it
		 * gets to actually comitting it. 
		 */
		uint64 ForceCommitImmediateTimeNanosec = 0;
		
		/** Tentative optimization recommended for SDK where a single big memblock is allocated. When false, a memblock per frame is configured. */
		bool bUseSingleMemblock = true;

		/** Whether to bump output thread priority to time critical */
		bool bEnableTimeCriticalThread = true;

		/** Whether to show output stats at regular interval in logs */
		bool bShowOutputStats = false;

		/** Interval in seconds at which to display output stats */
		float ShowOutputStatsIntervalSeconds = 1.0f;

		/** Whether to prefill RTP header memory with known data at initialization time instead of during sending */
		bool bPrefillRTPHeaders = true;

		/** Whether the stream should track and report when certain chunks (such as first chunk) were put on the wire. */
		bool bTrackChunkCompletion = false;
	};

	/** The base class for all Rivermax stream types. */
	class FRivermaxOutStream : public UE::RivermaxCore::IRivermaxOutputStream, public FRunnable
	{
	public:
		FRivermaxOutStream(const TArray<char>& SDPDescription);
		virtual ~FRivermaxOutStream();

	public:

		//~ Begin IRivermaxOutputStream interface
		virtual bool Initialize(const FRivermaxOutputOptions& Options, IRivermaxOutputStreamListener& InListener) override;
		virtual void Uninitialize() override;

		virtual bool IsGPUDirectSupported() const override;
		virtual bool ReserveFrame(uint64 FrameCounter) const override;
		virtual void GetLastPresentedFrame(FPresentedFrameInfo& OutFrameInfo) const override;
		//~ End IRivermaxOutputStream interface

		virtual void Process_AnyThread();

		//~ Begin FRunnable interface
		virtual bool Init() override;
		virtual uint32 Run() override;
		virtual void Stop() override;
		virtual void Exit() override;
		//~ End FRunnable interface

	protected:

		/** Configures chunks, packetizing, memory blocks of the stream */
		virtual bool InitializeStreamMemoryConfig() = 0;

		/** Resets NextFrame to be ready to send it out */
		void InitializeNextFrame(const TSharedPtr<FRivermaxOutputFrame>& NextFrame);

		/** Indicates that there is some data that is ready to be sent*/
		virtual bool IsFrameAvailableToSend() = 0;

		/** Query rivermax library for the next chunk to work with */
		virtual void GetNextChunk();

		/** Fills RTP header for all packets to be sent for this chunk */
		virtual void SetupRTPHeadersForChunk() = 0;

		/** Copies part of frame memory in next memblock's chunk to be sent out */
		virtual bool CopyFrameData(const TSharedPtr<FRivermaxOutputFrame>& SourceFrame, uint8* DestinationBase) = 0;

		/** Commits chunk to rivermax so they are scheduled to be sent */
		virtual void CommitNextChunks();

		/** Fetches next frame to send and prepares it for sending */
		virtual void PrepareNextFrame();

		/** Get next frame to be sent, stream type agnostic. Should wait until the frame is available if bWait. */
		virtual TSharedPtr<FRivermaxOutputFrame> GetNextFrameToSend(bool bWait = false) = 0;

		/** Returns next frame to send for frame creation alignment */
		virtual void PrepareNextFrame_FrameCreation();

		/** Returns next frame to send for alignement point method. Can return nullptr */
		virtual void PrepareNextFrame_AlignmentPoint();

		/** Sets up frame management taking care of allocation, special cuda handling, etc... */
		virtual bool SetupFrameManagement() = 0;

		/** Clean up frames */
		virtual void CleanupFrameManagement();

		/** Destroys rivermax stream. Will wait until it's ready to be destroyed */
		virtual void DestroyStream();

		/** When a frame has been sent (after frame interval), we update last presented frame tracking and optionally release it in the presentation queue */
		virtual void CompleteCurrentFrame(bool bReleaseFrame);

		/** Waits for the next point in time to send out a new frame. Returns true if it exited earlier with the next frame ready to be processed */
		virtual bool WaitForNextRound();

		/** Uses time before next frame interval to copy data from next ready frame to intermediate buffer */
		virtual void PreprocessNextFrame();

		/** Initializes timing setup for this stream. TRO, frame interval etc... */
		virtual void InitializeStreamTimingSettings();

		/** Get frame rate according to the SDP file. */
		virtual const FFrameRate& GetFrameRate() const;

	protected:

		/** Get stream address according to the SDP file. */
		virtual const FString& GetStreamAddress() const;

		/** Get Interface Address (Physical Port address). */
		virtual const FString& GetInterfaceAddress() const;

		/** Get Port Number according to the SDP file. */
		virtual uint32 GetPort() const;

		/** Get Stream Index as it is ordered in the SDP file. */
		virtual uint64 GetStreamIndexSDP() const;

		/** When stream creation is successful this should log all relevant information. */
		virtual void LogStreamDescriptionOnCreation() const = 0;

	protected:

		/** Calculate next frame scheduling time for alignment points mode */
		void CalculateNextScheduleTime_AlignementPoints(uint64 CurrentClockTimeNanosec, uint64 CurrentFrameNumber);
		
		/** Calculate next frame scheduling time for frame creation mode */
		void CalculateNextScheduleTime_FrameCreation(uint64 CurrentClockTimeNanosec, uint64 CurrentFrameNumber);

		/** Validates timing on every commit to see if we are respecting alignment */
		bool IsChunkOnTime() const;
		
		/** Validates timing for frame creation alignment which always returns true. */
		bool IsChunkOnTime_FrameCreation() const;
		
		/** Validates timing to make sure chunk to be committed are on time. 
		 *  Once a chunk is late, timings are at risk and next frame will be skipped
		 */
		bool IsChunkOnTime_AlignmentPoints() const;

		/** If enabled, print stats related to this stream */
		void ShowStats();

		/** Used to notify the listener that a frame is ready to be enqueued for transmission */
		void OnPreFrameReadyToBeSent();
		
		/** Used to detect when a frame is now ready to be sent */
		void OnFrameReadyToBeSent();

		/** Used to know when a frame is ready to be used and receive new data */
		void OnFrameReadyToBeUsed();

		/** Used to detect when the frame manager has caught a critical error */
		void OnFrameManagerCriticalError();

		/** Used to cache cvars at initialization */
		void CacheCVarValues();

		/** Called back when copy request was completed by allocator */
		void OnMemoryChunksCopied(const TSharedPtr<FBaseDataCopySideCar>& Sidecar);

		/** Called when delay request cvar has been changed */
		void OnCVarRandomDelayChanged(IConsoleVariable* Var);

		/** Called when Trigger Timing issue cvar value has been changed */
		void OnCVarTriggerTimingIssueChanged(IConsoleVariable* Var);

		/** Update frame's timestamp to be used when setting every RTP headers */
		void CalculateFrameTimestamp();

		/** Tells Rivermax to skip a certain number of chunks in memory. Can be zero to just reset internals */
		void SkipChunks(uint64 ChunkCount);

		/** Go through all chunks of current frame and commit them to Rivermax to send them at the next desired time */
		void SendFrame();

	protected:

		/** Rivermax memory configuration. i.e. memblock, chunks */
		FRivermaxOutputStreamMemory StreamMemory;

		/** Options related to this stream. i.e resolution, frame rate, etc... */
		FRivermaxOutputOptions Options;

		/** Various stats collected by this stream */
		FRivermaxOutputStreamStats Stats;

		/** State of various piece for this stream. Alignment points, schedule number, etc... */
		FRivermaxOutputStreamData StreamData;

		/** Stream id returned by rmax library */
		rmx_stream_id StreamId = 0;

		/** Current frame being sent */
		TSharedPtr<FRivermaxOutputFrame> CurrentFrame;

		/** 
		* Frames reserved when Media Capture texture is about to be converted into Buffer to be sent.
		* Key is frame counter
		*/
		mutable TMap<uint64, TSharedPtr<FRivermaxOutputFrame>> ReservedFrames;

		/** Thread scheduling frame output */
		TUniquePtr<FRunnableThread> RivermaxThread;

		/** Whether stream is active or not */
		std::atomic<bool> bIsActive;

		/** Event used to let scheduler that a frame is ready to be sent */
		FEventRef FrameReadyToSendSignal = FEventRef(EEventMode::AutoReset);

		/** Event used to unblock frame reservation as soon as one is free */
		FEventRef FrameAvailableSignal = FEventRef(EEventMode::AutoReset);

		/** Listener for this stream events */
		IRivermaxOutputStreamListener* Listener = nullptr;

		/** Type of stream created. */
		ERivermaxStreamType StreamType = ERivermaxStreamType::ST2110_20;

		/** TRoffset time calculated based on ST2110 - 21 Gapped(for now) method. This is added to next alignment point */
		uint64 TransmitOffsetNanosec = 0;

		/** Timestamp at which we logged stats */
		double LastStatsShownTimestamp = 0.0;
		
		/** Whether stream is using gpudirect to host memory consumed by Rivermax */
		bool bUseGPUDirect = false;

		/** Our own module pointer kept for ease of use */
		IRivermaxCoreModule* RivermaxModule = nullptr;

		/** Guid given by boundary monitoring handler to unregister ourselves */
		// TODO: There must be a better place for this.
		FGuid MonitoringGuid;

		/** Cached cvar values */
		FOutputStreamCachedCVars CachedCVars;

		/* Pointer to the rivermax API to avoid virtual calls in a hot loop. */ 
		const RIVERMAX_API_FUNCTION_LIST* CachedAPI = nullptr;

		/** Whether to trigger a delay in the output thread loop next time it ticks */
		bool bTriggerRandomDelay = false;

		/** Critical section to access data of last presented frame */
		mutable FCriticalSection PresentedFrameCS;

		/** Info of last presented frame */
		FPresentedFrameInfo LastPresentedFrame;

		/** An actual string of the SDP file. */
		TArray<char> SDPDescription;

	private:
		/** Used for debugging Rivermax streams when frame needs to be skipped after a certain period. */
		double ScheduledTimeForArtificialTimingIssue = 0.0;
	};
}


