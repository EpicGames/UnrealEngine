// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxOutVideoStream.h"

#include "Async/Async.h"
#if PLATFORM_WINDOWS
#include "CudaModule.h"
#endif // PLATFORM_WINDOWS
#include "IRivermaxCoreModule.h"
#include "IRivermaxBoundaryMonitor.h"
#include "IRivermaxManager.h"
#include "Misc/ByteSwap.h"
#include "RivermaxFrameAllocator.h"
#include "RivermaxFrameManager.h"
#include "RivermaxLog.h"
#include "RivermaxPTPUtils.h"
#include "RivermaxTracingUtils.h"
#include "RivermaxTypes.h"
#include "RivermaxUtils.h"


// All our RTP headers are made out of structs with bit fields.
// This macro is only set to 0 in cases where we want to debug the packets or try FBigEndianHeaderPacker, otherwise it is always set to 1. 
#define RIVERMAX_USE_BIT_FIELDS 1

namespace UE::RivermaxCore::Private
{
	static TAutoConsoleVariable<int32> CVarRivermaxOutputEnableMultiSRD(
		TEXT("Rivermax.Output.EnableMultiSRD"), 0,
		TEXT("When enabled and if the row cannot be split evenly, non-uniform payloads will be used. The last packet for the frame will not be fully filled with data.\n"
			"If disabled, the payloads will be split evenly or the 2110 stream will be disabled."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputLinesPerChunk(
		TEXT("Rivermax.Output.LinesPerChunk"), 4,
		TEXT("Defines the number of lines to pack in a chunk. Higher number will increase latency"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputMaximizePacketSize(
		TEXT("Rivermax.Output.MaximizePacketSize"), 1,
		TEXT("Enables bigger packet sizes to maximize utilisation of potential UDP packet. If not enabled, packet size will be aligned with HD/4k sizes"),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputMaxFrameMemorySliceCount(
		TEXT("Rivermax.Output.FrameSliceCount"), 30,
		TEXT("Max number of memcopies done per frame when using intermediate buffer. As frame gets bigger, we can't do a single memcopy or timings will be broken. Can be smaller in order to fit inside chunk count."),
		ECVF_Default);

	static TAutoConsoleVariable<bool> CVarRivermaxOutputEnableIntermediateBuffer(
		TEXT("Rivermax.Output.Alignment.EnableIntermediateBuffer"), true,
		TEXT("Uses an intermediate buffer used by Rivermax when sending data out.\n")
		TEXT("During scheduling, captured frame data will be copied over intermediate buffer.\n")
		TEXT("Only applies to alignment points scheduling mode."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarRivermaxOutputMemcopyChunkSpacing(
		TEXT("Rivermax.Output.Scheduling.MemcopyChunkSpacing"), 10,
		TEXT("Number of chunks between each memcopy to help with timing for different frame format."),
		ECVF_Default);

	// TODO: To be removed or improved in 5.7. In 5.6.1 Media Capture will be responsible for padding the resolution in case row cannot be split into equal sized packets due to issues with Multi-srd.
	bool FindPayloadSize(uint32 InBytesPerLine, const uint32 PixelGroupSize, uint16& OutPayloadSize)
	{
		/** Maximum payload size in bytes that the plugin can send based on UDP max size and RTP header.  */
		static constexpr uint32 MaxPayloadSize = 1420;

		/** Smallest payload size (bytes) to use as a lower bound in search for a payload that can be equal across a line */
		static constexpr uint32 MinPayloadSize = 600;

		// For now, we only find a payload size that can be equal across one line
		// Support for last payload of a line being smaller is there but is causing issue
		// We fail to output if we receive a resolution for which we can't find an equal payload size
		int32 TestPoint = InBytesPerLine / MaxPayloadSize;
		if (TestPoint == 0)
		{
			if (InBytesPerLine > MinPayloadSize)
			{
				if (InBytesPerLine % PixelGroupSize == 0)
				{
					OutPayloadSize = InBytesPerLine;
					return true;
				}
			}
			return false;
		}

		while (true)
		{
			const int32 TestSize = InBytesPerLine / TestPoint;
			if (TestSize < MinPayloadSize)
			{
				break;
			}

			if (TestSize <= MaxPayloadSize)
			{
				if ((TestSize % PixelGroupSize) == 0 && (InBytesPerLine % TestPoint) == 0)
				{
					OutPayloadSize = TestSize;
					return true;
				}
			}

			++TestPoint;
		}

		return false;
	}

	void UpdateVideoRtpHeader
		( uint8* OutVideoHeader
		, uint32& OutPacketCounter
		, uint16& OutSRDOffset
		, uint32& OutLineNumber
		, uint32 InSequenceNumber
		, uint32 InTimeStamp
		, const FIntPoint& AlignedResolution
		, const FVideoFormatInfo& InFormatInfo
		, const FRivermaxOutputStreamMemory& InStreamMemory
	)
	{
		const uint8 MarkerBit = (OutPacketCounter + 1 == InStreamMemory.PacketsPerFrame);
		constexpr uint8 PayloadType = 96;

#if RIVERMAX_USE_BIT_FIELDS
		FVideoRTPHeader* VideoHeader = reinterpret_cast<FVideoRTPHeader*>(OutVideoHeader);
		*VideoHeader = {};
		VideoHeader->RTPHeader = {};
		VideoHeader->RTPHeader.Version = 2;
		VideoHeader->RTPHeader.PaddingBit = 0;
		VideoHeader->RTPHeader.ExtensionBit = 0;
		VideoHeader->RTPHeader.PayloadType = PayloadType; //Payload type should probably be inferred from SDP

		VideoHeader->RTPHeader.SequenceNumber = ByteSwap((uint16)(InSequenceNumber & 0xFFFF));
		VideoHeader->RTPHeader.Timestamp = ByteSwap(InTimeStamp);
		VideoHeader->RTPHeader.ExtendedSequenceNumber = ByteSwap((uint16)((InSequenceNumber >> 16) & 0xFFFF));

		//2110 specific header
		VideoHeader->RTPHeader.SynchronizationSource = ByteSwap((uint32)FRawRTPHeader::VideoSynchronizationSource);  // Should Unreal has its own synch source ID
		VideoHeader->RTPHeader.MarkerBit = MarkerBit; // last packet in frame (Marker)
		
#endif
		
#if RIVERMAX_PACKET_DEBUG
		uint8 HeaderMemory[FVideoRTPHeader::TwoSRDSize];
		FBigEndianHeaderPacker HeaderPacker(HeaderMemory, FVideoRTPHeader::TwoSRDSize, true);
		constexpr uint8 Version = 2;
		HeaderPacker.AddField(Version, 2 RIVERMAX_DEBUG_FIELD_NAME("V"));

		constexpr uint8 PaddingBit = 0;
		HeaderPacker.AddField(PaddingBit, 1 RIVERMAX_DEBUG_FIELD_NAME("P"));

		constexpr uint8 ExtensionBit = 0;
		HeaderPacker.AddField(ExtensionBit, 1 RIVERMAX_DEBUG_FIELD_NAME("X"));

		constexpr uint8 ContributingSourceCount = 0;
		HeaderPacker.AddField(ContributingSourceCount, 4 RIVERMAX_DEBUG_FIELD_NAME("CC"));

		// Is this the last packet.
		HeaderPacker.AddField(MarkerBit, 1 RIVERMAX_DEBUG_FIELD_NAME("M"));

		HeaderPacker.AddField(PayloadType, 7 RIVERMAX_DEBUG_FIELD_NAME("PT"));
		HeaderPacker.AddField(InSequenceNumber & 0xFFFF, 16 RIVERMAX_DEBUG_FIELD_NAME("SEQ"));
		HeaderPacker.AddField(InTimeStamp, 32 RIVERMAX_DEBUG_FIELD_NAME("Timestamp"));
		HeaderPacker.AddField(FRawRTPHeader::VideoSynchronizationSource, 32 RIVERMAX_DEBUG_FIELD_NAME("SSRC"));
		HeaderPacker.AddField((InSequenceNumber >> 16) & 0xFFFF, 16 RIVERMAX_DEBUG_FIELD_NAME("Extended Sequence Number"));
#endif
		{
			// Verify if payload size exceeds line 
			const uint32 CurrentPayloadSize = InStreamMemory.PayloadSizes[OutPacketCounter];

			const uint32 LineSizeOffset = ((OutSRDOffset / InFormatInfo.PixelGroupCoverage) * InFormatInfo.PixelGroupSize);
			const uint32 LineSize = ((AlignedResolution.X / InFormatInfo.PixelGroupCoverage) * InFormatInfo.PixelGroupSize);

			const uint16 SRD1Length = FMath::Min(LineSize - LineSizeOffset, CurrentPayloadSize);
			const uint16 SRD1PixelCount = SRD1Length / InFormatInfo.PixelGroupSize * InFormatInfo.PixelGroupCoverage;

			uint16 SRD2Length = (SRD1Length < CurrentPayloadSize) ? (CurrentPayloadSize - SRD1Length) : 0;

			ensureMsgf(
				((uint32)SRD1Length + (uint32)SRD2Length) == CurrentPayloadSize,
				TEXT("SRD lengths must sum to payload size. SRD1=%u SRD2=%u Payload=%u Line=%u Offset=%u"), SRD1Length, SRD2Length, CurrentPayloadSize, OutLineNumber, OutSRDOffset
			);

			ensureMsgf(
				(SRD1Length % InFormatInfo.PixelGroupSize) == 0 && (SRD2Length % InFormatInfo.PixelGroupSize) == 0,
				TEXT("SRD lengths must be pixelgroup aligned. SRD1=%u SRD2=%u PixelGroupSize=%u"), SRD1Length, SRD2Length, InFormatInfo.PixelGroupSize
			);

			// If SRD2 is needed on the last line, something upstream is wrong (payload sizing/header sizing mismatch).
			ensureMsgf(
				!(SRD2Length && OutLineNumber == ((uint32)AlignedResolution.Y - 1)),
				TEXT("Invalid packetization. Payload crosses past last line. SRD2=%u Line=%u Res=%dx%d Offset=%u Payload=%u"),
				SRD2Length,
				OutLineNumber,
				AlignedResolution.X, AlignedResolution.Y,
				OutSRDOffset,
				CurrentPayloadSize
			);

			constexpr uint32 FieldIdentification = 0; //todo when fields are sent for interlace

#if RIVERMAX_USE_BIT_FIELDS
			VideoHeader->SRD1.Length = ByteSwap(SRD1Length);
			VideoHeader->SRD1.SetRowNumber(OutLineNumber); //todo divide by 2 if interlaced
			VideoHeader->SRD1.SetOffset(OutSRDOffset);
			VideoHeader->SRD1.ContinuationBit = (SRD2Length > 0);
			VideoHeader->SRD1.FieldIdentification = FieldIdentification;
#endif

#if RIVERMAX_PACKET_DEBUG
			HeaderPacker.AddField(SRD1Length, 16 RIVERMAX_DEBUG_FIELD_NAME("SRD Length"));

			HeaderPacker.AddField(FieldIdentification, 1 RIVERMAX_DEBUG_FIELD_NAME("F"));
			HeaderPacker.AddField(OutLineNumber, 15 RIVERMAX_DEBUG_FIELD_NAME("SRD Row Number"));

			// ContinuationBit - C
			const uint32 C = SRD2Length > 0;
			HeaderPacker.AddField(C, 1 RIVERMAX_DEBUG_FIELD_NAME("C"));
			HeaderPacker.AddField(OutSRDOffset, 15 RIVERMAX_DEBUG_FIELD_NAME("SRD Offset"));
#endif

			OutSRDOffset += SRD1PixelCount;
			if (OutSRDOffset >= AlignedResolution.X)
			{
				OutSRDOffset = 0;
				++OutLineNumber;
			}

			if (SRD2Length > 0)
			{

#if RIVERMAX_PACKET_DEBUG
				HeaderPacker.AddField(SRD2Length, 16 RIVERMAX_DEBUG_FIELD_NAME("SRD Length"));
				HeaderPacker.AddField(FieldIdentification, 1 RIVERMAX_DEBUG_FIELD_NAME("F"));
				HeaderPacker.AddField(OutLineNumber, 15 RIVERMAX_DEBUG_FIELD_NAME("SRD Row Number"));

				// ContinuationBit - C
				const uint32 C2 = 0;
				HeaderPacker.AddField(C2, 1 RIVERMAX_DEBUG_FIELD_NAME("C"));
				HeaderPacker.AddField(OutSRDOffset, 15 RIVERMAX_DEBUG_FIELD_NAME("SRD Offset"));
#endif

#if RIVERMAX_USE_BIT_FIELDS
				VideoHeader->SRD2.Length = ByteSwap(SRD2Length);
				VideoHeader->SRD2.SetRowNumber(OutLineNumber);
				VideoHeader->SRD2.SetOffset(OutSRDOffset);
				VideoHeader->SRD2.ContinuationBit = 0;
				VideoHeader->SRD2.FieldIdentification = FieldIdentification;
#endif

				const uint16 SRD2PixelCount = SRD2Length / InFormatInfo.PixelGroupSize * InFormatInfo.PixelGroupCoverage;
				OutSRDOffset += SRD2PixelCount;
				if (OutSRDOffset >= AlignedResolution.X)
				{
					OutSRDOffset = 0;
					++OutLineNumber;
				}
			}
			// Never happened. Sanity check.
			ensureMsgf(
				OutLineNumber <= (uint32)AlignedResolution.Y,
				TEXT("LineNumber overflow. Line=%u ResY=%d"), OutLineNumber, AlignedResolution.Y
			);
		}


#if RIVERMAX_PACKET_DEBUG
		HeaderPacker.Finalize();
#endif

// Unit test to run when switching platform to make sure that the bit fields are interpreted correctly.
#if RIVERMAX_USE_BIT_FIELDS & RIVERMAX_PACKET_DEBUG
		uint8* Ptr = (uint8*) VideoHeader;
		for (int64 Byte = 0; Byte < FVideoRTPHeader::TwoSRDSize; Byte++)
		{
			check(Ptr[Byte] == HeaderMemory[Byte]);
		}
#endif

#if !RIVERMAX_USE_BIT_FIELDS & RIVERMAX_PACKET_DEBUG
	// Since bit fields are not used and packet debugging is enabled we need to write data produced by header packer.
	memcpy(OutVideoHeader, HeaderMemory, FVideoRTPHeader::TwoSRDSize);
#endif
	}

	uint32 FindLinesPerChunk(const FRivermaxOutputOptions& InOptions)
	{
		// More lines per chunks mean we will do more work prior to start sending a chunk. So, added 'latency' in terms of packet / parts of frame.
		// Less lines per chunk mean that sender thread might starve.
		// SDK sample uses 4 lines for UHD and 8 for HD. 
		return CVarRivermaxOutputLinesPerChunk.GetValueOnAnyThread();
	}

	uint16 GetPayloadSize(ESamplingType SamplingType)
	{
		const FVideoFormatInfo FormatInfo = FStandardVideoFormat::GetVideoFormatInfo(SamplingType);
		uint16 PayloadSize;
		switch (SamplingType)
		{

		case ESamplingType::YUV444_10bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_10bit:
		{
			PayloadSize = 1200;
			break;
		}
		case ESamplingType::YUV444_8bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_8bit:
		{
			PayloadSize = 1152;
			break;
		}
		case ESamplingType::YUV444_12bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_12bit:
		{
			PayloadSize = 1152;
			break;
		}
		case ESamplingType::YUV444_16bit:
		{
			// Passthrough
		}
		case ESamplingType::YUV444_16bitFloat:
		{
			// Passthrough
		}
		case ESamplingType::RGB_16bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_16bitFloat:
		{
			PayloadSize = 1152;
			break;
		}
		case ESamplingType::YUV422_8bit:
		{
			PayloadSize = 1280;
			break;
		}
		case ESamplingType::YUV422_10bit:
		{
			PayloadSize = 1200;
			break;
		}
		case ESamplingType::YUV422_12bit:
		{
			PayloadSize = 1152;
			break;
		}
		case ESamplingType::YUV422_16bit:
		{
			// Passthrough
		}
		case ESamplingType::YUV422_16bitFloat:
		{
			PayloadSize = 1280;
			break;
		}
		default:
		{
			checkNoEntry();
			PayloadSize = 1200;
			break;
		}
		}

		ensure(PayloadSize % FormatInfo.PixelGroupSize == 0);
		return PayloadSize;
	}

	/**
	 * Returns a payload closer to the max value we can have for standard UDP size
	 * RTPHeader can be bigger depending on configuration so we'll cap payload at 1400.
	 */
	uint16 GetMaximizedPayloadSize(ESamplingType SamplingType)
	{
		const FVideoFormatInfo FormatInfo = FStandardVideoFormat::GetVideoFormatInfo(SamplingType);
		uint16 PayloadSize;
		switch (SamplingType)
		{

		case ESamplingType::YUV444_10bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_10bit:
		{
			PayloadSize = 1395;
			break;
		}
		case ESamplingType::YUV444_8bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_8bit:
		{
			PayloadSize = 1398;
			break;
		}
		case ESamplingType::YUV444_12bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_12bit:
		{
			PayloadSize = 1395;
			break;
		}
		case ESamplingType::YUV444_16bit:
		{
			// Passthrough
		}
		case ESamplingType::YUV444_16bitFloat:
		{
			// Passthrough
		}
		case ESamplingType::RGB_16bit:
		{
			// Passthrough
		}
		case ESamplingType::RGB_16bitFloat:
		{
			PayloadSize = 1398;
			break;
		}
		case ESamplingType::YUV422_8bit:
		{
			PayloadSize = 1400;
			break;
		}
		case ESamplingType::YUV422_10bit:
		{
			PayloadSize = 1400;
			break;
		}
		case ESamplingType::YUV422_12bit:
		{
			PayloadSize = 1398;
			break;
		}
		case ESamplingType::YUV422_16bit:
		{
			// Passthrough
		}
		case ESamplingType::YUV422_16bitFloat:
		{
			PayloadSize = 1400;
			break;
		}
		default:
		{
			checkNoEntry();
			PayloadSize = 1200;
			break;
		}
		}

		ensure(PayloadSize % FormatInfo.PixelGroupSize == 0);
		return PayloadSize;
	}


	FRivermaxOutputStreamVideo::FRivermaxOutputStreamVideo(const TArray<char>& SDPDescription)
		: FRivermaxOutStream(SDPDescription)
	{
	}

	struct FRTPHeaderPrefiller
	{
		FRTPHeaderPrefiller(FRivermaxOutputStreamVideo& InRmaxOutputStream)
			: Stream(InRmaxOutputStream)
		{
			RunningSRDOffsetPerFrame.SetNumZeroed(Stream.Options.NumberOfBuffers);
			RunningLineNumberPerFrame.SetNumZeroed(Stream.Options.NumberOfBuffers);
		}

		void Update(uint32 PacketIndex)
		{
			using namespace UE::RivermaxCore::Private::Utils;

			const int32 PacketCount = Stream.StreamMemory.ChunksPerFrameField * Stream.StreamMemory.PacketsPerChunk;
			uint32 BufferIndex = 0;
			for (int32 MemblockIndex = 0; MemblockIndex < Stream.StreamMemory.RTPHeaders.Num(); ++MemblockIndex)
			{
				for (uint32 FrameInBlockIndex = 0; FrameInBlockIndex < Stream.StreamMemory.FramesFieldPerMemoryBlock; ++FrameInBlockIndex)
				{
					uint16& SRDOffset = RunningSRDOffsetPerFrame[BufferIndex];
					uint32& LineNumber = RunningLineNumberPerFrame[BufferIndex];
					const uint32 HeaderIndex = PacketIndex + (FrameInBlockIndex * PacketCount);
					if (ensure(HeaderIndex < Stream.StreamMemory.PacketsPerMemoryBlock))
					{
						uint8* HeaderPtr = Stream.StreamMemory.RTPHeaders[MemblockIndex].Get() + HeaderIndex * Stream.StreamMemory.HeaderStrideSize;

						UpdateVideoRtpHeader(
							HeaderPtr,
							PacketIndex,
							SRDOffset,
							LineNumber,
							0, //SequenceNumber
							0, //Timestamp
							Stream.Options.GetStreamOptions<FRivermaxVideoOutputOptions>(Stream.StreamType)->AlignedResolution,
							Stream.FormatInfo,
							Stream.StreamMemory
						);
					}

					++BufferIndex;
				}
			}
		}

	private:
		TArray<uint16> RunningSRDOffsetPerFrame;
		TArray<uint32> RunningLineNumberPerFrame;
		FRivermaxOutputStreamVideo& Stream;
	};

	bool FRivermaxOutputStreamVideo::PushFrame(TSharedPtr<IRivermaxOutputInfo> FrameInfo)
	{
		if (!bIsActive)
		{
			return false;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutputStreamVideo::PushVideoFrame);
		TSharedPtr<FRivermaxOutputInfoVideo> NewFrameInfo = StaticCastSharedPtr<FRivermaxOutputInfoVideo>(FrameInfo);
		NewFrameInfo->Stride = GetRowSizeInBytes();
		NewFrameInfo->Height = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(StreamType)->AlignedResolution.Y;

		// Clear reserved frame if there is one. If not, try to get free frame.
		// ReservedFrame should always be valid when block on reservation mode is used.
		TSharedPtr<FRivermaxOutputFrame> ReservedFrame;
		if (!ReservedFrames.RemoveAndCopyValue(FrameInfo->FrameIdentifier, ReservedFrame))
		{
			check(Options.FrameLockingMode != EFrameLockingMode::BlockOnReservation);
			ReservedFrame = FrameManager->GetFreeFrame();
		}

		// If this is invalid it means that frame locking mode is BlockOnReservation and the render ran faster than media output fps.
		if (ReservedFrame.IsValid())
		{
			return FrameManager->SetFrameData(NewFrameInfo, ReservedFrame);
		}
		
		return false;
	}

	bool FRivermaxOutputStreamVideo::InitializeStreamMemoryConfig()
	{
		using namespace UE::RivermaxCore::Private::Utils;

		TSharedPtr<FRivermaxVideoOutputOptions> StreamOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(StreamType);
		FormatInfo = FStandardVideoFormat::GetVideoFormatInfo(StreamOptions->PixelFormat);

		// Verify resolution for sampling type
		if (StreamOptions->AlignedResolution.X % FormatInfo.PixelGroupCoverage != 0)
		{
			UE_LOGF(LogRivermax, Warning, "Can't create Rivermax Output Stream. Aligned horizontal resolution of %d doesn't align with pixel group coverage of %d.", StreamOptions->AlignedResolution.X, FormatInfo.PixelGroupCoverage);
			return false;
		}

		const SIZE_T BytesPerRow = GetRowSizeInBytes();
		const SIZE_T FrameSize = BytesPerRow * StreamOptions->AlignedResolution.Y;

		if (FrameSize == 0)
		{
			UE_LOGF(LogRivermax, Warning, "Can't create Rivermax Output Stream. Frame size of 0 is invalid. Verify resolution.");
			return false;
		}
		


		// By default we want to divide the bytes evenly across packets. Some resolutions will require packets to be sized unevenly.
		const bool bFoundPayload = FindPayloadSize(BytesPerRow, FormatInfo.PixelGroupSize, StreamMemory.PayloadSize);
		if (bFoundPayload == false)
		{
			// Find out payload we want to use. Either we go the 'potential' multi SRD route or we restrict the stream based on supported resolutions.
			if (CVarRivermaxOutputEnableMultiSRD.GetValueOnAnyThread() >= 1)
			{
				UE_LOGF(LogRivermax, Log, "Due to resolution %dx%d, row data will be sent over multiple packets with varied sizes.", StreamOptions->AlignedResolution.X, StreamOptions->AlignedResolution.Y);
				if (CVarRivermaxOutputMaximizePacketSize.GetValueOnAnyThread() >= 1)
				{
					StreamMemory.PayloadSize = GetMaximizedPayloadSize(FormatInfo.Sampling);
				}
				else
				{
					StreamMemory.PayloadSize = GetPayloadSize(FormatInfo.Sampling);
				}
			}
			else
			{
				UE_LOGF(LogRivermax, Warning, "Could not find payload size for desired resolution %dx%d for desired pixel format."
					"If the intention is to use non standard resolutions, users might want to enable multi-srd support via Rivermax.Output.EnableMultiSRD.", StreamOptions->AlignedResolution.X, StreamOptions->AlignedResolution.Y);
				return false;
			}
		}

		// With payload size in hand, figure out how many packets we will need, how many chunks (group of packets) and configure descriptor arrays
		const uint32 PixelCount = StreamOptions->AlignedResolution.X * StreamOptions->AlignedResolution.Y;
		const uint64 FrameSizeInBytes = PixelCount / FormatInfo.PixelGroupCoverage * FormatInfo.PixelGroupSize;

		// Sanity check. By this point, frame size must be pixel group aligned.
		ensureMsgf(
			(FrameSizeInBytes % FormatInfo.PixelGroupSize) == 0,
			TEXT("FrameSizeInBytes (%llu) must be pixel-group aligned (PixelGroupSize=%u). Res=%dx%d Sampling=%s"),
			FrameSizeInBytes,
			FormatInfo.PixelGroupSize,
			StreamOptions->AlignedResolution.X,
			StreamOptions->AlignedResolution.Y,
			LexToString(FormatInfo.Sampling)
		);

		StreamMemory.PixelGroupPerPacket = StreamMemory.PayloadSize / FormatInfo.PixelGroupSize;
		StreamMemory.PixelsPerPacket = StreamMemory.PixelGroupPerPacket * FormatInfo.PixelGroupCoverage;

		// We might need a smaller packet to complete the end of frame so ceil to the next value
		StreamMemory.PacketsPerFrame = FMath::CeilToInt32((float)PixelCount / StreamMemory.PixelsPerPacket);

		// Depending on resolution and payload size, last packet of a line might not be fully utilized but we need the remaining bytes so ceil to next value
		StreamMemory.PacketsInLine = FMath::CeilToInt32((float)StreamMemory.PacketsPerFrame / StreamOptions->AlignedResolution.Y);

		StreamMemory.LinesInChunk = FindLinesPerChunk(Options);
		StreamMemory.PacketsPerChunk = StreamMemory.LinesInChunk * StreamMemory.PacketsInLine;
		StreamMemory.FramesFieldPerMemoryBlock = CachedCVars.bUseSingleMemblock ? Options.NumberOfBuffers : 1;

		// Chunk count won't necessarily align with the number of packets required. We need an integer amount of chunks to initialize our stream
		// and calculate how many packets that represents. Rivermax will expect the payload/header array to be that size. It just means that
		// we will mark the extra packets as 0 size.
		StreamMemory.ChunksPerFrameField = FMath::CeilToInt32((float)StreamMemory.PacketsPerFrame / StreamMemory.PacketsPerChunk);
		const uint64 RealPacketsPerFrame = StreamMemory.ChunksPerFrameField * StreamMemory.PacketsPerChunk;
		StreamMemory.PacketsPerMemoryBlock = RealPacketsPerFrame * StreamMemory.FramesFieldPerMemoryBlock;
		StreamMemory.ChunksPerMemoryBlock = StreamMemory.FramesFieldPerMemoryBlock * StreamMemory.ChunksPerFrameField;
		StreamMemory.MemoryBlockCount = Options.NumberOfBuffers / StreamMemory.FramesFieldPerMemoryBlock;

		// Setup arrays with the right sizes so we can give pointers to rivermax. This makes the stream header sizes static.
		StreamMemory.RTPHeaders.SetNumZeroed(StreamMemory.MemoryBlockCount);
		StreamMemory.PayloadSizes.SetNumZeroed(StreamMemory.PacketsPerMemoryBlock);
		StreamMemory.HeaderSizes.SetNumZeroed(StreamMemory.PacketsPerMemoryBlock);

		// Packed size should be 26 octets.
		StreamMemory.HeaderStrideSize = FVideoRTPHeader::TwoSRDSize;

		if (!SetupFrameManagement())
		{
			return false;
		}

		FRTPHeaderPrefiller RTPFiller(*this);

		// Used to keep track of acquired frames to prevent the same frame fomr being acquired from the pool more than once. 
		// When this scope is exited all frames are returned back to the Frame Manager pool.
		TArray<TSharedPtr<UE::RivermaxCore::Private::FRivermaxOutputFrame>> ProcessedFrames;

		StreamMemory.MemoryBlocks.SetNumZeroed(StreamMemory.MemoryBlockCount);
		CachedAPI->rmx_output_media_init_mem_blocks(StreamMemory.MemoryBlocks.GetData(), StreamMemory.MemoryBlockCount);
		for (uint32 BlockIndex = 0; BlockIndex < StreamMemory.MemoryBlockCount; ++BlockIndex)
		{
			rmx_output_media_mem_block& Block = StreamMemory.MemoryBlocks[BlockIndex];
			CachedAPI->rmx_output_media_set_chunk_count(&Block, StreamMemory.ChunksPerMemoryBlock);

			// We have two sub block, header and data
			constexpr uint8 SubBlockCount = 2;
			CachedAPI->rmx_output_media_set_sub_block_count(&Block, SubBlockCount);

			// Describe Header block
			CachedAPI->rmx_output_media_set_packet_layout(&Block, StreamMemory.HeaderBlockID, StreamMemory.HeaderSizes.GetData());

			// Describe Data block
			CachedAPI->rmx_output_media_set_packet_layout(&Block, StreamMemory.DataBlockID, StreamMemory.PayloadSizes.GetData());

			rmx_mem_multi_key_region* DataMemory;
			DataMemory = CachedAPI->rmx_output_media_get_dup_sub_block(&Block, StreamMemory.DataBlockID);
			if (DataMemory == nullptr)
			{
				UE_LOGF(LogRivermax, Warning, "Faild to get payload memory block. Output stream won't be created.");
				return false;
			}

			// If intermediate buffer is used, we setup rmax memblock to use that address. Otherwise, we map it to our actual frame's address
			if (StreamMemory.bUseIntermediateBuffer)
			{
				DataMemory->addr = Allocator->GetFrameAddress(BlockIndex);
			}
			else
			{
				TSharedPtr<UE::RivermaxCore::Private::FRivermaxOutputFrame> ProcessedFrame = FrameManager->GetFreeFrame();
				DataMemory->addr = ProcessedFrame->Buffer;
				ProcessedFrames.Add(ProcessedFrame);
			}

			DataMemory->length = StreamMemory.ChunksPerFrameField * StreamMemory.PacketsPerChunk * StreamMemory.PayloadSize;

			constexpr rmx_mkey_id InvalidKey = ((rmx_mkey_id)(-1L));
			DataMemory->mkey[0] = InvalidKey;
			DataMemory->mkey[1] = InvalidKey;


			rmx_mem_multi_key_region* HeaderMemory;
			HeaderMemory = CachedAPI->rmx_output_media_get_dup_sub_block(&Block, StreamMemory.HeaderBlockID);
			if (HeaderMemory == nullptr)
			{
				UE_LOGF(LogRivermax, Warning, "Failed to get header memory block. Output stream won't be created.");
				return false;
			}
			StreamMemory.RTPHeaders[BlockIndex] = MakeShareable(
				(uint8*)FMemory::MallocZeroed(StreamMemory.PacketsPerMemoryBlock * StreamMemory.HeaderStrideSize),
				[](uint8* Ptr) { FMemory::Free(Ptr); }
			);
			HeaderMemory->addr = StreamMemory.RTPHeaders[BlockIndex].Get();
			HeaderMemory->length = StreamMemory.HeaderStrideSize;
			HeaderMemory->mkey[0] = InvalidKey;
			HeaderMemory->mkey[1] = InvalidKey;
		}

		uint64 ProcessedBytes = 0;
		uint32 LineSize = 0; // bytes consumed in current line (0..BytesPerRow)

		const uint32 BytesPerRowU32 = (uint32)BytesPerRow;

		for (uint32 PacketIndex = 0; PacketIndex < RealPacketsPerFrame; ++PacketIndex)
		{
			uint32 HeaderSize = 0;
			uint32 ThisPayloadSize = 0;

			if (ProcessedBytes < FrameSizeInBytes)
			{
				// 1. Decide actual payload size for this packet (full or last remainder)
				ThisPayloadSize = StreamMemory.PayloadSize;
				if ((ProcessedBytes + ThisPayloadSize) > FrameSizeInBytes)
				{
					ThisPayloadSize = (uint32)(FrameSizeInBytes - ProcessedBytes);
				}

				// Payload must always align to pixel groups and be a multiple of header defined payload size. 
				ensureMsgf(
					(ThisPayloadSize % FormatInfo.PixelGroupSize) == 0,
					TEXT("Payload size (%u) must be pixel-group aligned (PixelGroupSize=%u). Packet=%u ProcessedBytes=%llu FrameSizeInBytes=%llu"),
					ThisPayloadSize,
					FormatInfo.PixelGroupSize,
					PacketIndex,
					ProcessedBytes,
					FrameSizeInBytes
				);

				// 2. Decide SRD count using THIS payload size
				const uint32 RemainingInLine = BytesPerRowU32 - LineSize;

				if (ThisPayloadSize <= RemainingInLine)
				{
					HeaderSize = FVideoRTPHeader::OneSRDSize;

					LineSize += ThisPayloadSize;
					if (LineSize == BytesPerRowU32)
					{
						LineSize = 0;
					}
				}
				else
				{
					HeaderSize = FVideoRTPHeader::TwoSRDSize;

					const uint32 Spill = ThisPayloadSize - RemainingInLine;
					LineSize = Spill;

					// Would require more than two SRDs, which is unsupported
					if (LineSize > BytesPerRowU32)
					{
						UE_LOGF(LogRivermax, Warning,
							"Unsupported resolution/payload: packet crosses more than 2 lines (needs >2 SRD). Res=%dx%d Spill=%u BytesPerRow=%u Payload=%u",
							StreamOptions->AlignedResolution.X, StreamOptions->AlignedResolution.Y, LineSize, BytesPerRowU32, ThisPayloadSize);
						return false;
					}
				}
			}
			else
			{
				// Extra packet slots for chunk alignment.
				HeaderSize = 0;
				ThisPayloadSize = 0;
			}

			// All buffers configured the same
			for (uint32 BufferIndex = 0; BufferIndex < StreamMemory.FramesFieldPerMemoryBlock; ++BufferIndex)
			{
				StreamMemory.HeaderSizes[PacketIndex + (BufferIndex * RealPacketsPerFrame)] = HeaderSize;
				StreamMemory.PayloadSizes[PacketIndex + (BufferIndex * RealPacketsPerFrame)] = ThisPayloadSize;
			}

			if (CachedCVars.bPrefillRTPHeaders && HeaderSize > 0)
			{
				RTPFiller.Update(PacketIndex);
			}

			ProcessedBytes += ThisPayloadSize;
		}

		// Verify memcopy config to make sure it works for current frame size / chunking
		if (StreamMemory.bUseIntermediateBuffer)
		{
			StreamMemory.FrameMemorySliceCount = FMath::Clamp(CVarRivermaxOutputMaxFrameMemorySliceCount.GetValueOnAnyThread(), 1, 100);
			StreamMemory.ChunkSpacingBetweenMemcopies = FMath::Clamp(CVarRivermaxOutputMemcopyChunkSpacing.GetValueOnAnyThread(), 1, 20);

			const uint32 ChunkRequired = StreamMemory.ChunkSpacingBetweenMemcopies * StreamMemory.FrameMemorySliceCount;
			if (ChunkRequired > 0 && ChunkRequired > StreamMemory.ChunksPerFrameField)
			{
				// Favor reducing number of memcopies. If required packet is smaller, chances are it's a small frame size
				// so memcopies will be smaller.
				const double Ratio = StreamMemory.ChunksPerFrameField / (double)ChunkRequired;
				StreamMemory.FrameMemorySliceCount *= Ratio;
			}
		}

		return true;
	}

	bool FRivermaxOutputStreamVideo::IsFrameAvailableToSend()
	{
		return FrameManager->IsFrameAvailableToSend();
	}

	bool FRivermaxOutputStreamVideo::CopyFrameData(const TSharedPtr<FRivermaxOutputFrame>& SourceFrame, uint8* DestinationBase)
	{
		// Make sure copy size doesn't go over frame size
		const SIZE_T FrameSize = GetRowSizeInBytes() * Options.GetStreamOptions<FRivermaxVideoOutputOptions>(StreamType)->AlignedResolution.Y;
		if (ensure(FrameSize > 0))
		{
			const SIZE_T BlockSize = 1 + ((FrameSize - 1) / StreamMemory.FrameMemorySliceCount);
			const SIZE_T MaxSize = FrameSize - SourceFrame->Offset;
			const SIZE_T CopySize = FMath::Min(BlockSize, MaxSize);

			// Copy data until we have covered the whole frame. Last block might be smaller.
			if (CopySize > 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(CopyFrameData);
				const SIZE_T PacketsToCopy = CopySize / StreamMemory.PayloadSize;
				const SIZE_T PacketOffset = SourceFrame->ChunkNumber * StreamMemory.PacketsPerChunk;
				const uint64 PacketDeltaTime = (StreamData.FrameFieldTimeIntervalNs - TransmitOffsetNanosec) / StreamMemory.PacketsPerFrame;

				FCopyArgs Args;
				uint8* SourceStart = reinterpret_cast<uint8*>(SourceFrame->Buffer);
				uint8* DestinationStart = DestinationBase;
				Args.SourceMemory = SourceStart + SourceFrame->Offset;
				Args.DestinationMemory = DestinationStart + SourceFrame->Offset;
				Args.SizeToCopy = CopySize;

				// Update memory offset for next copy
				SourceFrame->Offset += CopySize;

				Allocator->CopyData(Args);

				return true;
			}
		}
		else
		{
			UE_LOGF(LogRivermax, Error, "Invalid frame size detected while stream was active. Shutting down.");
			Listener->OnStreamError();
			Stop();
		}

		return false;
	}

	bool FRivermaxOutputStreamVideo::SetupFrameManagement()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRivermaxOutputStreamVideo::SetupFrameManagement);

		FrameManager = MakeUnique<FFrameManager>();
		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = Options.GetStreamOptions<FRivermaxVideoOutputOptions>(StreamType);

		// We do (try) to make gpu allocations here to let the capturer know if we require it or not.
		bool bTryGPUDirect = RivermaxModule->GetRivermaxManager()->IsGPUDirectOutputSupported() && VideoOptions->bUseGPUDirect;
		if (bTryGPUDirect)
		{
			const ERHIInterfaceType RHIType = RHIGetInterfaceType();
			if (RHIType != ERHIInterfaceType::D3D12)
			{
				UE_LOGF(LogRivermax, Warning, "Can't initialize output to use GPUDirect. RHI is %d but only Dx12 is supported at the moment.", EnumToUnderlyingType(RHIType));
				bTryGPUDirect = false;
			}
		}

		// Work around when dealing with multi memblocks. Rivermax fails to create stream 
		// With memblock not starting on the right cuda alignment. 
		const bool bAlignEachFrameMemory = !CachedCVars.bUseSingleMemblock;
		FFrameManagerSetupArgs FrameManagerArgs;
		FrameManagerArgs.Resolution = VideoOptions->AlignedResolution;
		FrameManagerArgs.bTryGPUAllocation = bTryGPUDirect;
		FrameManagerArgs.NumberOfFrames = Options.NumberOfBuffers;
		FrameManagerArgs.Stride = GetRowSizeInBytes();
		FrameManagerArgs.FrameDesiredSize = StreamMemory.ChunksPerFrameField * StreamMemory.PacketsPerChunk * StreamMemory.PayloadSize;
		FrameManagerArgs.bAlignEachFrameAlloc = bAlignEachFrameMemory;
		FrameManagerArgs.OnFreeFrameDelegate = FOnFrameReadyDelegate::CreateRaw(this, &FRivermaxOutputStreamVideo::OnFrameReadyToBeUsed);
		FrameManagerArgs.OnPreFrameReadyDelegate = FOnFrameReadyDelegate::CreateRaw(this, &FRivermaxOutputStreamVideo::OnPreFrameReadyToBeSent);
		FrameManagerArgs.OnFrameReadyDelegate = FOnFrameReadyDelegate::CreateRaw(this, &FRivermaxOutputStreamVideo::OnFrameReadyToBeSent);
		FrameManagerArgs.OnCriticalErrorDelegate = FOnCriticalErrorDelegate::CreateRaw(this, &FRivermaxOutputStreamVideo::OnFrameManagerCriticalError);
		const EFrameMemoryLocation FrameLocation = FrameManager->Initialize(FrameManagerArgs);
		bUseGPUDirect = FrameLocation == EFrameMemoryLocation::GPU;


		// Only support intermediate buffer for alignment point method to avoid running into chunk issue when repeating a frame
		const bool bHasAllocatedFrames = (FrameLocation != EFrameMemoryLocation::None);
		if (bHasAllocatedFrames && Options.AlignmentMode == ERivermaxAlignmentMode::AlignmentPoint && CVarRivermaxOutputEnableIntermediateBuffer.GetValueOnAnyThread())
		{
			// Allocate intermediate buffer in same memory space as frame memory.
			FOnFrameDataCopiedDelegate OnDataCopiedDelegate = FOnFrameDataCopiedDelegate::CreateRaw(this, &FRivermaxOutputStreamVideo::OnMemoryChunksCopied);

			const int32 DesiredSize = FrameManagerArgs.FrameDesiredSize * StreamMemory.FramesFieldPerMemoryBlock;
#if PLATFORM_WINDOWS
			if (bUseGPUDirect)
			{
				Allocator = MakeUnique<FGPUAllocator>(DesiredSize, OnDataCopiedDelegate);
			}
			else
#endif // PLATFORM_WINDOWS
			{
				Allocator = MakeUnique<FSystemAllocator>(DesiredSize, OnDataCopiedDelegate);
			}

			if (!Allocator->Allocate(StreamMemory.MemoryBlockCount, bAlignEachFrameMemory))
			{
				return false;
			}

			StreamMemory.bUseIntermediateBuffer = true;
		}

		// Cache buffer addresses used by rivermax in order to start copying into it early on
		StreamMemory.BufferAddresses.Reserve(Options.NumberOfBuffers);
		if (StreamMemory.bUseIntermediateBuffer)
		{
			check(Allocator);

			for (uint32 MemblockIndex = 0; MemblockIndex < StreamMemory.MemoryBlockCount; ++MemblockIndex)
			{
				uint8* BaseAddress = static_cast<uint8*>(Allocator->GetFrameAddress(MemblockIndex));
				for (uint32 FrameIndex = 0; FrameIndex < StreamMemory.FramesFieldPerMemoryBlock; ++FrameIndex)
				{
					uint8* FrameAddress = BaseAddress + (FrameIndex * FrameManagerArgs.FrameDesiredSize);
					StreamMemory.BufferAddresses.Add(FrameAddress);
				}
			}
		}
		else
		{
			// Used to keep track of acquired frames to prevent the same frame from being acquired from the pool more than once. 
			// When this scope is exited all frames are returned back to the Frame Manager pool.
			TArray<TSharedPtr<UE::RivermaxCore::Private::FRivermaxOutputFrame>> ProcessedFrames;

			// When we don't use intermediate buffer, each frame has its own buffer addresses and we don't need to look at memblocks
			for (int32 BufferIndex = 0; BufferIndex < Options.NumberOfBuffers; ++BufferIndex)
			{
				TSharedPtr<UE::RivermaxCore::Private::FRivermaxOutputFrame> Frame = FrameManager->GetFreeFrame();
				StreamMemory.BufferAddresses.Add(Frame->Buffer);
				ProcessedFrames.Add(Frame);
			}
		}
		check(StreamMemory.BufferAddresses.Num() == Options.NumberOfBuffers);

		return FrameLocation != EFrameMemoryLocation::None;
	}

	void FRivermaxOutputStreamVideo::CleanupFrameManagement()
	{
		if (FrameManager)
		{
			FrameManager->Cleanup();
			FrameManager.Reset();
		}

		if (Allocator)
		{
			Allocator->Deallocate();
			Allocator.Reset();
		}
		FRivermaxOutStream::CleanupFrameManagement();
	}

	SIZE_T FRivermaxOutputStreamVideo::GetRowSizeInBytes() const
	{
		check(FormatInfo.PixelGroupCoverage != 0);
		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = StaticCastSharedPtr<FRivermaxVideoOutputOptions>(Options.StreamOptions[(uint8)StreamType]);
		return (VideoOptions->AlignedResolution.X / FormatInfo.PixelGroupCoverage) * FormatInfo.PixelGroupSize;
	}

	void FRivermaxOutputStreamVideo::SetupRTPHeadersForChunk()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::SetupRTPHeadersForChunk);
		uint8* FirstHeaderPtr = reinterpret_cast<uint8*>(CurrentFrame->HeaderPtr);
		check(FirstHeaderPtr);

		for (uint32 PacketIndex = 0; PacketIndex < StreamMemory.PacketsPerChunk && CurrentFrame->PacketCounter < StreamMemory.PacketsPerFrame; ++PacketIndex)
		{
			using namespace UE::RivermaxCore::Private::Utils;

			uint8* DataRawPtr = FirstHeaderPtr + PacketIndex * StreamMemory.HeaderStrideSize;

			// Since the static part of the RTP header is filled on initalization we only need to update the 
			// non static parts.
			if (CachedCVars.bPrefillRTPHeaders)
			{
#if RIVERMAX_USE_BIT_FIELDS
				FVideoRTPHeader* HeaderRawPtr = reinterpret_cast<FVideoRTPHeader*>(DataRawPtr);
				HeaderRawPtr->RTPHeader.SequenceNumber = ByteSwap((uint16)(StreamData.SequenceNumber & 0xFFFF));
				HeaderRawPtr->RTPHeader.Timestamp = ByteSwap(CurrentFrame->MediaTimestamp);
				HeaderRawPtr->RTPHeader.ExtendedSequenceNumber = ByteSwap((uint16)((StreamData.SequenceNumber >> 16) & 0xFFFF));
#else
				FBigEndianHeaderPacker HeaderPacker(DataRawPtr, FVideoRTPHeader::TwoSRDSize, false /*bClearExisting*/);
				HeaderPacker.Finalize();
				HeaderPacker.UpdateField((StreamData.SequenceNumber & 0xFFFF), 16, 16 RIVERMAX_DEBUG_FIELD_NAME("SEQ"));
				HeaderPacker.UpdateField(CurrentFrame->MediaTimestamp, 32, 32 RIVERMAX_DEBUG_FIELD_NAME("Timestamp"));
				HeaderPacker.UpdateField((StreamData.SequenceNumber >> 16) & 0xFFFF, 16, 96 RIVERMAX_DEBUG_FIELD_NAME("Extended Sequence Number"));
#endif
			}
			else
			{
				TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = StaticCastSharedPtr<FRivermaxVideoOutputOptions>(Options.StreamOptions[(uint8)StreamType]);
				UpdateVideoRtpHeader(
					DataRawPtr,
					CurrentFrame->PacketCounter,
					CurrentFrame->SRDOffset,
					CurrentFrame->LineNumber,
					StreamData.SequenceNumber,
					CurrentFrame->MediaTimestamp,
					VideoOptions->AlignedResolution,
					FormatInfo,
					StreamMemory
				);
			}

			++StreamData.SequenceNumber;
			++CurrentFrame->PacketCounter;
		}

		//UE_LOGF(LogRivermax, Log, "Packets Processed: %d, Packets per chunk: %d, ChunkNum: %d", PacketIndex, StreamMemory.PacketsPerChunk, CurrentFrame->ChunkNumber);

	}

	void FRivermaxOutputStreamVideo::LogStreamDescriptionOnCreation() const
	{
		FRivermaxOutStream::LogStreamDescriptionOnCreation();

		TSharedPtr<FRivermaxVideoOutputOptions> VideoOptions = StaticCastSharedPtr<FRivermaxVideoOutputOptions>(Options.StreamOptions[(uint8)StreamType]);

		TStringBuilder<512> StreamDescription;
		if (bUseGPUDirect)
		{
			StreamDescription.Appendf(TEXT("Using GPUDirect."));
		}

		StreamDescription.Appendf(TEXT("Settings: Resolution = %dx%d, "), VideoOptions->AlignedResolution.X, VideoOptions->AlignedResolution.Y);
		StreamDescription.Appendf(TEXT("FrameRate = %s, "), *VideoOptions->FrameRate.ToPrettyText().ToString());
		StreamDescription.Appendf(TEXT("Pixel format = %s, "), LexToString(VideoOptions->PixelFormat));
		StreamDescription.Appendf(TEXT("Alignment = %s, "), LexToString(Options.AlignmentMode));
		StreamDescription.Appendf(TEXT("Framelocking = %s."), LexToString(Options.FrameLockingMode));

		UE_LOGF(LogRivermax, Display, "%ls", *FString(StreamDescription));
	}

	bool FRivermaxOutputStreamVideo::ReserveFrame(uint64 FrameCounter) const
	{
		if (!bIsActive)
		{
			return false;
		}

		// There is only one reserved frame at the time per stream.
		TSharedPtr<FRivermaxOutputFrame> ReservedFrame = FrameManager->GetFreeFrame();
		if (!ReservedFrame.IsValid() && Options.FrameLockingMode == EFrameLockingMode::BlockOnReservation)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Rmax::WaitForAvailableFrame);
			while (!ReservedFrame && bIsActive)
			{
				FrameAvailableSignal->Wait();
				ReservedFrame = FrameManager->GetFreeFrame();
			}
		}

		if (ReservedFrame.IsValid() && bIsActive)
		{
			ReservedFrame->SetFrameCounter(FrameCounter);
			ReservedFrames.Add(FrameCounter, ReservedFrame);
		}

		return ReservedFrame.IsValid();
	}

	TSharedPtr<FRivermaxOutputFrame> FRivermaxOutputStreamVideo::GetNextFrameToSend(bool bWait)
	{
		TSharedPtr<FRivermaxOutputFrame> NextFrameToSend = FrameManager->DequeueFrameToSend();

		if (bWait)
		{
			while (!NextFrameToSend.IsValid() && bIsActive)
			{
				FrameReadyToSendSignal->Wait();
				NextFrameToSend = FrameManager->DequeueFrameToSend();
			}
		}
		return NextFrameToSend;
	}

	void FRivermaxOutputStreamVideo::CompleteCurrentFrame(bool bReleaseFrame)
	{
		FRivermaxOutStream::CompleteCurrentFrame(bReleaseFrame);

		// We don't release when there is no new frame, so we keep a hold on it to repeat it.
		if (bReleaseFrame)
		{
			FrameManager->FrameSentEvent();
		}
	}
}

