// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RivermaxOutStream.h"

#include "Async/Future.h"
#include "Containers/SpscQueue.h"
#include "HAL/Runnable.h"
#include "RivermaxWrapper.h"
#include "RivermaxOutputFrame.h"
#include "RivermaxTypes.h"
#include "RTPHeader.h"


class FEvent;
class IRivermaxCoreModule;

namespace UE::RivermaxCore::Private
{
	class FFrameManager;
	class FBaseFrameAllocator;

	using UE::RivermaxCore::FRivermaxOutputOptions;

	class FRivermaxOutputStreamVideo : public FRivermaxOutStream
	{
	public:
		FRivermaxOutputStreamVideo(const TArray<char>& SDPDescription);

	public:

		//~ Begin IRivermaxOutputStream interface
		virtual bool PushFrame(TSharedPtr<IRivermaxOutputInfo> FrameInfo) override;
		virtual bool ReserveFrame(uint64 FrameIdentifier) const override;
		//~ End IRivermaxOutputStream interface

	protected:

		//~ Begin FRivermaxOutStream interface
		virtual bool IsFrameAvailableToSend();
		virtual bool InitializeStreamMemoryConfig();
		virtual bool CopyFrameData(const TSharedPtr<FRivermaxOutputFrame>& SourceFrame, uint8* DestinationBase);
		virtual bool SetupFrameManagement();
		virtual void SetupRTPHeadersForChunk() override;
		virtual void CleanupFrameManagement() override;
		virtual void CompleteCurrentFrame(bool bReleaseFrame) override;
		virtual void LogStreamDescriptionOnCreation() const override;
		virtual TSharedPtr<FRivermaxOutputFrame> GetNextFrameToSend(bool bWait = false);
		//~ End FRivermaxOutStream interface

	private:
		
		/** Get row stride for the current stream configuration */
		SIZE_T GetRowSizeInBytes() const;

	private:

		/** Manages allocation and memory manipulation of video frames */
		TUniquePtr<FFrameManager> FrameManager;

		/** Manages allocation of memory for rivermax memblocks */
		TUniquePtr<FBaseFrameAllocator> Allocator;

		/** Format info for the active stream */
		FVideoFormatInfo FormatInfo;

		// A helper struct that is hidden in cpp file and needs access to video stream options to fill the RTP packet.
		friend struct FRTPHeaderPrefiller;
	};
}


