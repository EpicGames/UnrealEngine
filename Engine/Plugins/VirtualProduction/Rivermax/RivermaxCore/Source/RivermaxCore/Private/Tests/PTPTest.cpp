// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Misc/AutomationTest.h"
#include "RivermaxPTPUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPTPTest, "Plugins.Rivermax.PTP", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

struct FTestData
{
	FFrameRate Rate;
	uint64 TimeNanosec = 0;
	uint64 ExpectedFrameNumber = 0;
	uint64 ExpectedAlignmentPointNanosec = 0;
};

bool FPTPTest::RunTest(const FString& Parameters)
{
	TArray<FTestData> TestPoints;

	// Add couple of test points at different frame rate to test PTP operation

	TestPoints.Add({ FFrameRate(24,1), 0, 0, 41666667 });
	TestPoints.Add({ FFrameRate(24,1), 42000000, 1, 83333334 });
	TestPoints.Add({ FFrameRate(24,1), 1671408000000000000, 40113792000, 1671408000041666667 });
	TestPoints.Add({ FFrameRate(24,1), 1671408000041666666, 40113792000, 1671408000041666667 });
	TestPoints.Add({ FFrameRate(24,1), 1671408000041666667, 40113792001, 1671408000083333334 });
	TestPoints.Add({ FFrameRate(24,1), 1671408000083333334, 40113792002, 1671408000125000000 });
	TestPoints.Add({ FFrameRate(24,1), 1671408000020000000, 40113792000, 1671408000041666667 }); 
	TestPoints.Add({ FFrameRate(24,1), 1671408000040000000, 40113792000, 1671408000041666667 });
	TestPoints.Add({ FFrameRate(24,1), 1673079658458541667, 40153911803, 1673079658500000000 });
	TestPoints.Add({ FFrameRate(24,1), 4099680000000000000, 98392320000, 4099680000041666667 });
	TestPoints.Add({ FFrameRate(24,1), 4099680000041666667, 98392320001, 4099680000083333334 }); // Large nano second test. Approx 130 years from epoch

	TestPoints.Add({ FFrameRate(24000,1001), 0, 0, 41708334 }); 
	TestPoints.Add({ FFrameRate(24000,1001), 42000000, 1, 83416667 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408000000000, 40113792000, 1673079408041708334 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408020000000, 40113792000, 1673079408041708334 });  
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408040000000, 40113792000, 1673079408041708334 });  
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408041708334, 40113792001, 1673079408083416667 }); 
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408083416667, 40113792002, 1673079408125125000 }); 
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408125124999, 40113792002, 1673079408125125000 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408125125000, 40113792003, 1673079408166833334 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408166833334, 40113792004, 1673079408208541667 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408208541667, 40113792005, 1673079408250250000 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408250249998, 40113792005, 1673079408250250000 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408250250000, 40113792006, 1673079408291958334 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408291958334, 40113792007, 1673079408333666667 });
	TestPoints.Add({ FFrameRate(24000,1001), 1673079408333666667, 40113792008, 1673079408375375000 });
	TestPoints.Add({ FFrameRate(24000,1001), 4099680000000000000, 98294025974, 4099680000040625000 });
	TestPoints.Add({ FFrameRate(24000,1001), 4099680000040625000, 98294025975, 4099680000082333334 });
	TestPoints.Add({ FFrameRate(24000,1001), 4099680000082333334, 98294025976, 4099680000124041667 });

	TestPoints.Add({ FFrameRate(60,1), 0, 0, 16666667 });
	TestPoints.Add({ FFrameRate(60,1), 26666666, 1, 33333334 });
	TestPoints.Add({ FFrameRate(60,1), 2674252800000000000, 160455168000, 2674252800016666667 }); 
	TestPoints.Add({ FFrameRate(60,1), 2674252800008000000, 160455168000, 2674252800016666667 });
	TestPoints.Add({ FFrameRate(60,1), 2674252800010000000, 160455168000, 2674252800016666667 });
	TestPoints.Add({ FFrameRate(60,1), 1673079658458541667, 100384779507, 1673079658466666667 });
	TestPoints.Add({ FFrameRate(60,1), 1673079658466666667, 100384779508, 1673079658483333334 });
	TestPoints.Add({ FFrameRate(60,1), 4099680000000000000, 245980800000, 4099680000016666667 });
	TestPoints.Add({ FFrameRate(60,1), 4099680000016666666, 245980800000, 4099680000016666667 });
	TestPoints.Add({ FFrameRate(60,1), 4099680000016666667, 245980800001, 4099680000033333334 });

	TestPoints.Add({ FFrameRate(60000,1001), 0, 0, 16683334 });
	TestPoints.Add({ FFrameRate(60000,1001), 26666666, 1, 33366667 });
	TestPoints.Add({ FFrameRate(60000,1001), 2674252800000000000, 160294873126, 2674252800002116667 });
	TestPoints.Add({ FFrameRate(60000,1001), 2674252800008000000, 160294873127, 2674252800018800000 });
	TestPoints.Add({ FFrameRate(60000,1001), 2674252800010000000, 160294873127, 2674252800018800000 });
	TestPoints.Add({ FFrameRate(60000,1001), 1673079658458541667, 100284495012, 1673079658466883334 });
	TestPoints.Add({ FFrameRate(60000,1001), 1673079658466883334, 100284495013, 1673079658483566667 });
	TestPoints.Add({ FFrameRate(60000,1001), 4099680000000000000, 245735064935, 4099680000015600000 });
	

	const auto AlignmentTestFunc = [this](const FTestData& InTestPoint)
	{
		const uint64 FoundFrameNumber = UE::RivermaxCore::GetFrameNumber(InTestPoint.TimeNanosec, InTestPoint.Rate);
		const FString FrameNumberError = FString::Printf(TEXT("Frame rate %s: Error calculating frame number for time %llu."), *InTestPoint.Rate.ToPrettyText().ToString(), InTestPoint.TimeNanosec);
		const FString AlignmentPointError = FString::Printf(TEXT("Frame rate %s: Error calculating next alignment point for time %llu."), *InTestPoint.Rate.ToPrettyText().ToString(), InTestPoint.TimeNanosec);
		TestEqual(*FrameNumberError, FoundFrameNumber, InTestPoint.ExpectedFrameNumber);
		const uint64 NextAlignmentPointNanosec = UE::RivermaxCore::GetNextAlignmentPoint(InTestPoint.TimeNanosec, InTestPoint.Rate);
		TestEqual(*AlignmentPointError, NextAlignmentPointNanosec, InTestPoint.ExpectedAlignmentPointNanosec);

		return true;
	};

	for (const FTestData& TestPoint : TestPoints)
	{
		AlignmentTestFunc(TestPoint);
	}

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
