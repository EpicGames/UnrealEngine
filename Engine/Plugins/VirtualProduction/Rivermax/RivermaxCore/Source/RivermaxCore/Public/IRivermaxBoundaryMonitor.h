// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct FGuid;
struct FFrameRate;

namespace UE::RivermaxCore
{
	/** 
	 * Frame boundary monitor interface used to
	 * add trace bookmarks for monitored frame rates.
	 * Boundaries are calculated using ST-2059 formula.
	 */
	class RIVERMAXCORE_API IRivermaxBoundaryMonitor
	{
	public:
		virtual ~IRivermaxBoundaryMonitor() = default;

		/** Global enable / disable of monitoring thread */
		virtual void EnableMonitoring(bool bEnable) = 0;

		/** Asks to start monitoring a particular frame rate */
		virtual FGuid StartMonitoring(const FFrameRate& FrameRate) = 0;
	
		/** Removes a listener for a particular frame rate */
		virtual void StopMonitoring(const FGuid& Requester, const FFrameRate& FrameRate) = 0;
	};
}

