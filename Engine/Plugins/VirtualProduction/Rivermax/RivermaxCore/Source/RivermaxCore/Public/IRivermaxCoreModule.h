// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "IRivermaxInputStream.h"
#include "IRivermaxOutputStream.h"
#include "Modules/ModuleManager.h"
#include "RivermaxTypes.h"


namespace UE::RivermaxCore
{
	class IRivermaxManager;
	class IRivermaxBoundaryMonitor;
}


/**
 * Core module for Rivermax access from the engine. Users can create different stream types that are exposed to 
 * get data flow ongoing.
 */
class IRivermaxCoreModule : public IModuleInterface
{
public:
	static inline IRivermaxCoreModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IRivermaxCoreModule>("RivermaxCore");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("RivermaxCore");
	}

	UE_DEPRECATED(5.6, "Please use CreateInputStream that takes stream type as a parameter.")
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxInputStream> CreateInputStream() { return nullptr; };

	UE_DEPRECATED(5.6, "Please use CreateOutputStream that takes stream type as a parameter.")
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream> CreateOutputStream() { return nullptr; };

	/** Create input stream managing receiving data from rivermax */
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxInputStream> CreateInputStream(UE::RivermaxCore::ERivermaxStreamType, const TArray<char>& InSDPDescription) = 0;

	/** Create output stream managing sending data to rivermax */
	virtual TUniquePtr<UE::RivermaxCore::IRivermaxOutputStream> CreateOutputStream(UE::RivermaxCore::ERivermaxStreamType, const TArray<char>& InSDPDescription) = 0;

	/** Returns Rivermax manager singleton to query for stats, status, etc... */
	virtual TSharedPtr<UE::RivermaxCore::IRivermaxManager> GetRivermaxManager() = 0;

	/** Returns frame boundary monitor */
	virtual UE::RivermaxCore::IRivermaxBoundaryMonitor& GetRivermaxBoundaryMonitor() = 0;
};

