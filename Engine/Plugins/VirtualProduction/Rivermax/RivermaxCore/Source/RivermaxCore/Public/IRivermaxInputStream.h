// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RivermaxFormats.h"

class FRHIBuffer;
class FRDGPooledBuffer;

namespace UE::RivermaxCore
{
	struct FRivermaxInputStreamOptions;

	struct RIVERMAXCORE_API FRivermaxInputInitializationResult
	{
		/** Whether initialization suceeded */
		bool bHasSucceed = false;

		/** Whether gpudirect can be used, if requested in the first place */
		bool bIsGPUDirectSupported = false;
	};

	struct RIVERMAXCORE_API FRivermaxInputVideoFrameDescriptor
	{
		/** Height of the received frame */
		uint32 Height = 0;

		/** Width of the received frame */
		uint32 Width = 0;

		/** Total size of the video frame */
		uint32 VideoBufferSize = 0;

		/** Timestamp, in media clock realm, marked by the sender */
		uint32 Timestamp = 0;

		/** Frame number derived from timestamp and frame rate */
		uint32 FrameNumber = 0;

		/** Pixel format of the frame */
		ESamplingType PixelFormat = ESamplingType::RGB_10bit;
	};

	struct RIVERMAXCORE_API FRivermaxInputVideoFormatChangedInfo
	{
		/** Detected height in pixel of the video stream */
		uint32 Height = 0;

		/** Detected width in pixel of the video stream */
		uint32 Width = 0;

		/** Detected sampling type of the video stream */
		ESamplingType PixelFormat = ESamplingType::RGB_10bit;
	};

	struct RIVERMAXCORE_API FRivermaxInputVideoFrameRequest
	{
		/** Buffer pointer in RAM where to write incoming frame */
		uint8* VideoBuffer = nullptr;
		
		/** Buffer pointer in GPU to be mapped to cuda and where to write incoming frame */
		FRHIBuffer* GPUBuffer = nullptr;
	};

	struct RIVERMAXCORE_API FRivermaxInputVideoFrameReception
	{
		uint8* VideoBuffer = nullptr;
	};

	/** 
	* All types of streams write into corresponding sample types. All sample types should inherit from this class so it can be correctly cast and identified.
	*/
	class RIVERMAXCORE_API IRivermaxSample
	{
	public:
		virtual ~IRivermaxSample()
		{

		}
		enum class ESampleState : uint8
		{
			// Sample hasn't started receiving and is awaiting to be used. In this state it contains the data previously written into it.
			Idle,

			// Sample is ready to be written into.
			ReadyForReception,

			// Sample has received the data. And ready to be used for rendering.
			Received,

			// Error receiving sample. Used at the same point as Received state.
			ReceptionError,

			// After sample is received and is ready to be rendered and until the GPU is done with the sample this is the state the sample is in. 
			// Currently only used for debugging purposes.
			Rendering
		};

		/**
		* Enum that identifies the type of this sample such as Video or Audio so that it can be handled accordingly.
		*/
		enum class ESampleType : uint8
		{
			// 2110-20
			Video,

			// 2110-30
			Audio,

			// 2110-40
			Anc,

			// 2110-20 Sub raster.
			KeyAndFill,

			// Stub for itteration.
			MAX
		};

	public:

		/**
		* Sets the state of the reception for this sample.
		*/
		virtual void SetReceptionState(ESampleState State)
		{
			FScopeLock Lock(&StateChangeCriticalSecion);
			SampleState = State;
		}

		/**
		* Gets reception state of this sample.
		*/
		virtual ESampleState GetReceptionState()
		{
			FScopeLock Lock(&StateChangeCriticalSecion);
			return SampleState;
		}

	protected:
		/** State of this sample. */
		std::atomic<ESampleState> SampleState = ESampleState::ReadyForReception;

		/** Type of the sample. */
		std::atomic<ESampleType> SampleType = ESampleType::Video;

		/** Critical section used when manipulating the received/skipped/rendered and other states in this or the inheriting classes. */
		mutable FCriticalSection StateChangeCriticalSecion;
	};

	/**
	* 2110-20 sample type interface.
	*/
	class RIVERMAXCORE_API IRivermaxVideoSample : public IRivermaxSample
	{
	public:
		
		/** Returns RDG allocated buffer */
		virtual TRefCountPtr<FRDGPooledBuffer> GetGPUBuffer() const = 0;

		/** Returns a pointer to the CPU accessible buffer for writing streams into. */
		virtual uint8* GetVideoBufferRawPtr(uint32 VideoBufferSize) = 0;
	};

	/**
	* 2110-40 ANC sample type interface.
	*/
	class RIVERMAXCORE_API IRivermaxAncSample : public IRivermaxSample
	{
	public:
		virtual void SetAncData(uint16 DID, uint16 SDID, const TArray<uint16>& UDWs) = 0;
		virtual uint16 GetDID() const = 0;
		virtual uint16 GetSDID() const = 0;
		virtual const TArray<uint16>& GetUserDataWords() const = 0;
	};

	/** Descriptor filled during ANC packet reception, passed to OnAncFrameRequested. */
	struct RIVERMAXCORE_API FRivermaxInputAncFrameDescriptor
	{
		uint32 Timestamp = 0;
		uint32 FrameNumber = 0;
		uint16 DID = 0;
		uint16 SDID = 0;
		uint16 LineNumber = 0;
		uint16 HorizontalOffset = 0;
	};

	/**
	* A type of interface that is able to provide samples for data to be written into and react to stream events such as completion of the reception.
	*/
	class RIVERMAXCORE_API IRivermaxInputStreamListener
	{
	public:
		/** Initialization completion callback with result */
		virtual void OnInitializationCompleted(const FRivermaxInputInitializationResult& Result) = 0;

		/** Called when stream is ready to fill the next frame. Returns true if a frame was successfully requested */
		virtual TSharedPtr<IRivermaxVideoSample> OnVideoFrameRequested(const FRivermaxInputVideoFrameDescriptor& FrameInfo) = 0;

		/** Called when a frame has been received */
		virtual void OnVideoFrameReceived(TSharedPtr<IRivermaxVideoSample> InReceivedVideoFrame) = 0;

		/** Called when an error was encountered during frame reception */
		virtual void OnVideoFrameReceptionError(TSharedPtr<IRivermaxVideoSample> InVideoFrameSample) {};

		/** Called when stream has encountered an error and has to stop */
		virtual void OnStreamError() = 0;

		/**  Called when stream has detected a change in the video format */
		virtual void OnVideoFormatChanged(const FRivermaxInputVideoFormatChangedInfo& NewFormatInfo) = 0;

		/** Called when an ANC stream is ready to receive a packet. Return a valid sample to accept it. */
		virtual TSharedPtr<IRivermaxAncSample> OnAncFrameRequested(const FRivermaxInputAncFrameDescriptor& FrameInfo)
		{
			return nullptr;
		}

		/** Called when an ANC packet has been fully received and parsed into InReceivedAncFrame. */
		virtual void OnAncFrameReceived(TSharedPtr<IRivermaxAncSample> InReceivedAncFrame)
		{
		}

		/** Called when a received ANC packet could not be parsed (checksum error, truncated packet, etc.). */
		virtual void OnAncFrameReceptionError()
		{
		}
	};

	/**
	* Interface for initializing input stream from media player.
	*/
	class RIVERMAXCORE_API IRivermaxInputStream
	{
	public:
		virtual ~IRivermaxInputStream() = default;

	public:
		virtual bool Initialize(const FRivermaxInputStreamOptions& InOptions, IRivermaxInputStreamListener& InListener) = 0;
		virtual void Uninitialize() = 0;
	};
}

