// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RivermaxSettings.h"

namespace UE::RivermaxCore
{
	struct FRivermaxDeviceInfo
	{
		FString Description;
		FString InterfaceAddress;
	};

	namespace Private
	{
		struct RIVERMAX_API_FUNCTION_LIST;
	}

	/** Delegate called when rivermax manager has been initialized and library tried to be loaded */
	DECLARE_MULTICAST_DELEGATE(FOnPostRivermaxManagerInit)

	/**
	 * Not doing much at the moment but should be used as a central point to register every stream created
	 * and keep track of stats. Also manages initialization of the library.
	 */
	class RIVERMAXCORE_API IRivermaxManager
	{
	public:
		virtual ~IRivermaxManager() = default;

	public:
		
		/** Returns true if RivermaxManager has been initialized */
		virtual bool IsManagerInitialized() const = 0;

		/** Returns true if Rivermax library has been initialized and is usable */
		virtual bool IsLibraryInitialized() const = 0;

		/** Returns true if Rivermax library has been initialized and is usable, but also warns the user with a toast notification if it's not. */
		virtual bool ValidateLibraryIsLoaded() const = 0;

		/** Return the Rivermax API. */
		virtual const Private::RIVERMAX_API_FUNCTION_LIST* GetApi() const = 0;

		/** Delegate triggered after manager has been initialized and library tried to be loaded */
		virtual FOnPostRivermaxManagerInit& OnPostRivermaxManagerInit() = 0;

		/** 
		 * Returns current time in nanoseconds for Rivermax
		 * If PTP is available, it will be used
		 * If not, FPlatform::Seconds will be used
		 */
		virtual uint64 GetTime() const = 0;

		/** Returns where time is coming from */
		virtual ERivermaxTimeSource GetTimeSource() const = 0;

		/** Returns list of found devices that can be used */
		virtual TConstArrayView<FRivermaxDeviceInfo> GetDevices() const = 0;

		/** Returns a matching device interface address for the provided source IP */
		virtual bool GetMatchingDevice(const FString& InSourceIP, FString& OutDeviceIP) const = 0;

		/** Returns true if provided IP string is in a valid format */
		virtual bool IsValidIP(const FString& InSourceIP) const = 0;

		/** Whether gpudirect is supported on this platform. At the moment, we rely on CUDA to handle it. */
		virtual bool IsGPUDirectSupported() const = 0;

		/** Whether gpudirect is enabled for input streams. */
		virtual bool IsGPUDirectInputSupported() const = 0;

		/** Whether gpudirect is enabled for output streams. */
		virtual bool IsGPUDirectOutputSupported() const = 0;

		/** Enables dynamic header support for a particular Rivermax device. */
		virtual bool EnableDynamicHeaderSupport(const FString& Interface) = 0;

		/** If it's the last user of that interface, we disable dynamic header */
		virtual void DisableDynamicHeaderSupport(const FString& Interface) = 0;
	};
}

