// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Timecode.h"

#include "RHI.h"


namespace UE::RivermaxCore
{
	struct FRivermaxOutputOptions;

	/** Base struct with description of the frame to be captured, required by all stream types. */
	struct RIVERMAXCORE_API IRivermaxOutputInfo
	{
		/** Incremental number identifying frame. Usually GFrameCounter */
		uint64 FrameIdentifier = 0;

		/** Height of the frame */
		uint32 Height = 0;

		/** Width of the frame */
		uint32 Width = 0;

		virtual ~IRivermaxOutputInfo() {};
	};

	/** Description of video frame to be captured. */
	struct RIVERMAXCORE_API FRivermaxOutputInfoVideo : public IRivermaxOutputInfo
	{
		/** Video frame data location in system memory */
		void* CPUBuffer = nullptr;

		/** Video frame data location when using GPUDirect */
		FBufferRHIRef GPUBuffer;

		/** Stride of a line */
		uint32 Stride = 0;
	};
	
	struct RIVERMAXCORE_API FRivermaxOutputInfoAnc : public IRivermaxOutputInfo
	{
		/** User data words to be put on the wire. */
		TArray<uint16> UDWs;
	};

	/** Description of Anc timecode frame to be captured. */
	struct RIVERMAXCORE_API FRivermaxOutputInfoAncTimecode : public FRivermaxOutputInfoAnc
	{
		/** Stream frame rate. */
		FFrameRate FrameRate;

		/** Frame's timecode. */
		FTimecode Timecode;
	};

	/** Information about the last frame that was presented by the stream */
	struct RIVERMAXCORE_API FPresentedFrameInfo
	{
		/** Frame boundary at which the RenderedFrameNumber has been presented*/
		uint64 PresentedFrameBoundaryNumber = 0;

		/** Last engine's FrameNumber that was presented. */
		uint32 RenderedFrameNumber = 0;
	};

	class RIVERMAXCORE_API IRivermaxOutputStreamListener
	{
	public:

		/** Initialization completion callback with result */
		virtual void OnInitializationCompleted(bool bHasSucceed) = 0;

		/** Called when stream has encountered an error and has to stop */
		virtual void OnStreamError() = 0;

		/** Called when stream is about to enqueue new frame */
		virtual void OnPreFrameEnqueue() = 0;
	};

	class RIVERMAXCORE_API IRivermaxOutputStream
	{
	public:
		virtual ~IRivermaxOutputStream() = default;

	public:

		/** 
		 * Initializes stream using input options. Returns false if stream creation has failed. 
		 * Initialization is completed when OnInitializationCompletion has been called with result.
		 */
		virtual bool Initialize(const FRivermaxOutputOptions& Options, IRivermaxOutputStreamListener& InListener) = 0;

		/** Uninitializes current stream */
		virtual void Uninitialize() = 0;

		/** Pushes new video frame to the stream. Returns false if frame couldn't be pushed. */
		UE_DEPRECATED(5.6, "This method is deprecated. Please use the PushFrame.")
		virtual bool PushVideoFrame(const FRivermaxOutputInfoVideo& NewFrame) { return false; };

		/** Pushes new frame to the stream. Returns false if frame couldn't be pushed. */
		virtual bool PushFrame(TSharedPtr<IRivermaxOutputInfo> FrameInfo) = 0;

		/** Returns true if GPUDirect is supported */
		virtual bool IsGPUDirectSupported() const = 0;

		/** 
		 * Tries to reserve a frame for the next capture 
		 * If FRivermaxOutputOptions::FrameLockingMode equals BlockOnReservation, 
		 * this method will block until a free frame is found. 
		 */
		virtual bool ReserveFrame(uint64 FrameCounter) const = 0;

		/** Returns information about the last frame that was presented on the wire */
		virtual void GetLastPresentedFrame(FPresentedFrameInfo& OutFrameInfo) const = 0;
	};
}


