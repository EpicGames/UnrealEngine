// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/FrameRate.h"

namespace UE::RivermaxCore
{
	enum class ESamplingType : uint8
	{
		YUV422_8bit
		, YUV422_10bit
		, YUV422_12bit
		, YUV422_16bit
		, YUV422_16bitFloat
		, YUV444_8bit
		, YUV444_10bit
		, YUV444_12bit
		, YUV444_16bit
		, YUV444_16bitFloat
		, RGB_8bit
		, RGB_10bit
		, RGB_12bit
		, RGB_16bit
		, RGB_16bitFloat
	};

	inline const TCHAR* LexToString(ESamplingType InType)
	{
		switch (InType)
		{

		case ESamplingType::YUV444_10bit:
		{
			return TEXT("YUV444 10bit");
		}
		case ESamplingType::RGB_10bit:
		{
			return TEXT("RGB 10bit");
		}
		case ESamplingType::YUV444_8bit:
		{
			return TEXT("YUV444 8bit");
		}
		case ESamplingType::RGB_8bit:
		{
			return TEXT("RGB 8bit");
		}
		case ESamplingType::YUV444_12bit:
		{
			return TEXT("YUV444 12bit");
		}
		case ESamplingType::RGB_12bit:
		{
			return TEXT("RGB 12bit");
		}
		case ESamplingType::YUV444_16bit:
		{
			return TEXT("YUV444 16bit");
		}
		case ESamplingType::YUV444_16bitFloat:
		{
			return TEXT("YUV444 16bit float");
		}
		case ESamplingType::RGB_16bit:
		{
			return TEXT("RGB 16bit");
		}
		case ESamplingType::RGB_16bitFloat:
		{
			return TEXT("RGB 16bit float");
		}
		case ESamplingType::YUV422_8bit:
		{
			return TEXT("YUV422 8bit");
		}
		case ESamplingType::YUV422_10bit:
		{
			return TEXT("YUV422 10bit");
		}
		case ESamplingType::YUV422_12bit:
		{
			return TEXT("YUV422 12bit");
		}
		case ESamplingType::YUV422_16bit:
		{
			return TEXT("YUV422 16bit");
		}
		case ESamplingType::YUV422_16bitFloat:
		{
			return TEXT("YUV422 16bit float");
		}
		default:
		{
			checkNoEntry();
			break;
		}
		}

		return TEXT("<Unknown ESamplingType>");;
	}

	struct FVideoFormatInfo
	{
		/** Sampling type of this format */
		ESamplingType Sampling = ESamplingType::RGB_10bit;

		/** Bit depth for this format (i.e 8,10,12,16) */
		uint16 BitDepth = 0;

		/** Pixel Group (pgroup) size in bytes */
		uint16 PixelGroupSize = 0;

		/** Pixel Group coverage (pgroup coverage) in pixels */
		uint16 PixelGroupCoverage = 0;
	};

	struct FStandardVideoFormat
	{
		RIVERMAXCORE_API static FVideoFormatInfo GetVideoFormatInfo(ESamplingType SamplingType);
		RIVERMAXCORE_API static bool IsRGB(ESamplingType SamplingType);
		RIVERMAXCORE_API static bool IsYUV(ESamplingType SamplingType);
		RIVERMAXCORE_API static bool IsYUV422(ESamplingType SamplingType);
		RIVERMAXCORE_API static bool IsYUV444(ESamplingType SamplingType);

	private:
		static const TArray<FVideoFormatInfo> AllSamplingTypes;
	};
}


