// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Timecode.h"


struct FFrameRate;

namespace UE::RivermaxCore
{
	/** 
	 * Returns next alignment point in nanosecond based on incoming PTP time and frame rate.
	 * Follows ST2059 standard for alignment
	 */
	RIVERMAXCORE_API uint64 GetNextAlignmentPoint(const uint64 InPTPTimeNanosec, const FFrameRate& InRate);
	
	/**
	 * Returns alignment point in nanosecond based on incoming frame number and frame rate.
	 */
	RIVERMAXCORE_API uint64 GetAlignmentPointFromFrameNumber(const uint64 InFrameNumber, const FFrameRate& InRate);

	/**
	 * Returns current frame number for the incoming PTP time for the given frame rate.
	 */
	RIVERMAXCORE_API uint64 GetFrameNumber(const uint64 InPTPTimeNanosec, const FFrameRate& InRate);

	/** Decodes 16 ATC LTC UDWs (produced by TimecodeToAtcUDW10) back to FTimecode. */
	RIVERMAXCORE_API FTimecode AtcUDW10ToTimecode(const TArray<uint16>& UDWs);
}


