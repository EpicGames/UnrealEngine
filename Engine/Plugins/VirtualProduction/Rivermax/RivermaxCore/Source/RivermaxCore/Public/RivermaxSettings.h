// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "RivermaxSettings.generated.h"


UENUM()
enum class ERivermaxTimeSource
{
	PTP,
	Engine,
	System
};

/**
 * Settings for Rivermax core plugin. 
 */
UCLASS(config=Engine)
class RIVERMAXCORE_API URivermaxSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDevelopperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif
	//~ End UDevelopperSettings interface

public:

	/** Timing souce to be used by Rivermax scheduler */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta=(ConfigRestartRequired = true))
	ERivermaxTimeSource TimeSource = ERivermaxTimeSource::PTP;

	/** Interface to use when Rivermax timing is configured to use PTP */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (EditCondition = "TimeSource == ERivermaxTimeSource::PTP", ConfigRestartRequired = true))
	FString PTPInterfaceAddress = TEXT("*.*.*.*");
};