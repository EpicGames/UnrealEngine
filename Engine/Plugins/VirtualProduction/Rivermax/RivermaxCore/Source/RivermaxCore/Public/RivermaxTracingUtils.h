// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "CoreTypes.h"

namespace UE::RivermaxCore
{
	class RIVERMAXCORE_API FRivermaxTracingUtils
	{
	public:
		/** Media capture tracing string used to track a specific frame number being captured */
		static TMap<uint8, FString> RmaxOutMediaCapturePipeTraceEvents;

		/** Rivermax output stream sending markup */
		static TMap<uint8, FString> RmaxOutSendingFrameTraceEvents;

		/** Rivermax frame capture is ready markup */
		static TMap<uint8, FString> RmaxOutFrameReadyTraceEvents;

		/** Rivermax input stream starting to receive a frame markup */
		static TMap<uint8, FString> RmaxInStartingFrameTraceEvents;

		/** Rivermax input stream frame received markup */
		static TMap<uint8, FString> RmaxInReceivedFrameTraceEvents;

		/** Rivermax player frame selected markup */
		static TMap<uint8, FString> RmaxInSelectedFrameTraceEvents;

		/** Rivermax ANC output stream sending markup */
		static TMap<uint8, FString> RmaxOutAncSendingFrameTraceEvents;
	};
}


