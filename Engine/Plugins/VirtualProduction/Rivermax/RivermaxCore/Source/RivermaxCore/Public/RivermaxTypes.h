// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/FrameRate.h"
#include "RivermaxFormats.h"
#include "Containers/StaticArray.h"


namespace UE::RivermaxCore
{
	constexpr TCHAR DefaultStreamAddress[] = TEXT("228.1.1.1");


	enum class ERivermaxAlignmentMode
	{
		/** Aligns scheduling with ST2059 frame boundary formula */
		AlignmentPoint,

		/** Aligns scheduling with frame creation */
		FrameCreation,
	};

	inline const TCHAR* LexToString(ERivermaxAlignmentMode InValue)
	{
		switch (InValue)
		{
			case ERivermaxAlignmentMode::AlignmentPoint:
			{
				return TEXT("Alignment point");
			}
			case ERivermaxAlignmentMode::FrameCreation:
			{
				return TEXT("Frame creation");
			}
			default:
			{
				checkNoEntry();
			}
		}

		return TEXT("<Unknown ERivermaxAlignmentMode>");
	}

	enum class EFrameLockingMode : uint8
	{
		/** If no frame available, continue */
		FreeRun,

		/** Blocks when reserving a frame slot. */
		BlockOnReservation,
	};

	inline const TCHAR* LexToString(EFrameLockingMode InValue)
	{
		switch (InValue)
		{
			case EFrameLockingMode::FreeRun:
			{
				return TEXT("Freerun");
			}
			case EFrameLockingMode::BlockOnReservation:
			{
				return TEXT("Blocking");
			}
			default:
			{
				checkNoEntry();
			}
		}

		return TEXT("<Unknown EFrameLockingMode>");
	}

	struct FRivermaxInputStreamOptions
	{
		/** Stream FrameRate */
		FFrameRate FrameRate = { 24,1 };

		/** Interface IP to bind to */
		FString InterfaceAddress;

		/** IP of the stream. Defaults to multicast group IP. */
		FString StreamAddress = DefaultStreamAddress;

		/** Port to be used by stream */
		uint32 Port = 50000;

		/** Desired stream pixel format */
		ESamplingType PixelFormat = ESamplingType::RGB_10bit;

		/** Sample count to buffer. */
		int32 NumberOfBuffers = 2;
		
		/** If true, don't use auto detected video format */
		bool bEnforceVideoFormat = false;

		/** Enforced resolution aligning with pgroup of sampling type */
		FIntPoint EnforcedResolution = FIntPoint::ZeroValue;

		/** Whether to leverage GPUDirect (Cuda) capability to transfer memory to NIC if available */
		bool bUseGPUDirect = true;
	};

	struct FRivermaxOutputStreamOptions
	{
		/** Stream FrameRate */
		FFrameRate FrameRate = { 24,1 };

		/** Interface IP to bind to */
		FString InterfaceAddress;

		/** IP of the stream. Defaults to multicast group IP. */
		FString StreamAddress = DefaultStreamAddress;

		/** Port to be used by stream */
		uint32 Port = 50000;

		/** Used by RivermaxOutStream when it calls to the library to assign Media Block Index in SDP. */
		uint64 StreamIndex = 0;
	};

	struct FRivermaxVideoOutputOptions : public FRivermaxOutputStreamOptions
	{
		/** Desired stream resolution */
		FIntPoint Resolution = { 1920, 1080 };

		/** Desired stream pixel format */
		ESamplingType PixelFormat = ESamplingType::RGB_10bit;

		/** Resolution aligning with pgroup of sampling type */
		FIntPoint AlignedResolution = FIntPoint::ZeroValue;

		/** Whether to leverage GPUDirect (Cuda) capability to transfer memory to NIC if available */
		bool bUseGPUDirect = true;
	};

	struct RIVERMAXCORE_API FRivermaxAncOutputOptions : public FRivermaxOutputStreamOptions
	{
		/** This constructor is required to process and store DID and SDID according to ST291 standard to be ready to be used in RTP packet. */
		FRivermaxAncOutputOptions(uint8 InDID, uint8 InSDID);

		/** returns parity wrapped 10 bit DID value. */
		uint16 GetDID() const { return DID; }

		/** returns parity wrapped 10 bit SDID value. */
		uint16 GetSDID() const { return SDID; };

		/** returns raw DID value. */
		uint8 GetDIDRaw() const { return DIDRaw; }

		/** returns raw SDID value. */
		uint8 GetSDIDRaw() const { return SDIDRaw; }

	private:

		/**
		* User readable DID value as specified by ST291 https://www.smpte-ra.org/smpte-ancillary-data-smpte-st-291 
		*/
		uint8 DIDRaw = 0;

		/**
		* User readable SDID value as specified by ST291 https://www.smpte-ra.org/smpte-ancillary-data-smpte-st-291
		*/
		uint8 SDIDRaw = 0;

		/** 
		* DID value with Parity. For more information see MakeDataIdentificationWord.
		*/
		uint16 DID = 0;

		/** SDID value with Parity. For more information see MakeDataIdentificationWord. */
		uint16 SDID = 0;
	};

	enum ERivermaxStreamType : uint8
	{
		/** Video. */
		ST2110_20,
		/** Audio. */
		ST2110_30,
		/** General Anc. */
		ST2110_40,
		/** Timecode Anc. */
		ST2110_40_TC,
		MAX
	};

	struct FRivermaxOutputOptions
	{
		/** Contains a stream option for each ERivermaxStreamType. */
		TStaticArray<TSharedPtr<FRivermaxOutputStreamOptions>, static_cast<uint32>(ERivermaxStreamType::MAX)> StreamOptions;

		/** Sample count to buffer. */
		int32 NumberOfBuffers = 2;

		/** Method used to align output stream */
		ERivermaxAlignmentMode AlignmentMode = ERivermaxAlignmentMode::AlignmentPoint;

		/** Defines how frame requests are handled. Whether they can block or not. */
		EFrameLockingMode FrameLockingMode = EFrameLockingMode::FreeRun;

		/** Whether the stream will output a frame at every frame interval, repeating last frame if no new one provided */
		bool bDoContinuousOutput = true;

		/** Whether to use frame's frame number instead of standard timestamping */
		bool bDoFrameCounterTimestamping = true;

		/** Returns stream options for this stream. */
		template <typename T>
		TSharedPtr<T> GetStreamOptions(ERivermaxStreamType InStreamType)
		{
			return StaticCastSharedPtr<T>(StreamOptions[InStreamType]);
		}

		TSharedPtr<FRivermaxOutputStreamOptions> GetStreamOptions(ERivermaxStreamType InStreamType)
		{
			return StreamOptions[InStreamType];
		}
	};
}


