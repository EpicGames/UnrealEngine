// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RivermaxCore : ModuleRules
{
	public RivermaxCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"MediaAssets",
				"RHI",
				"RivermaxLib",
				"TimeManagement"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"Networking",
				"RenderCore",
				"Json"
			}
		);

		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				}
			);
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			//CUDA/GPU-direct are Windows-only for now
			PrivateDependencyModuleNames.Add("D3D12RHI");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "CUDA", "DX12");
		}

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Slate",
					"UnrealEd"
				}
			);

		}
	}
}
