// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/RivermaxDeviceSelectionCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Widgets/SRivermaxInterfaceComboBox.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

namespace UE::RivermaxCore::Utils
{
	void SetupDeviceSelectionCustomization(int32 InObjectIndex, const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder)
	{
		IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(PropertyHandle);

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		Row->GetDefaultWidgets(NameWidget, ValueWidget);
	
		Row->CustomWidget()
			.NameContent()
			[
				NameWidget->AsShared()
			]
			.ValueContent()
			[
				SNew(SRivermaxInterfaceComboBox)
				.InitialValue(InitialValue)
				.OnIPAddressSelected_Lambda([InObjectIndex, PropertyHandle](const FString& SelectedIp)
				{
						FScopedTransaction Tx(NSLOCTEXT("Rivermax", "ChangeInterfaceAddress", "Change Interface Address"));

						const FPropertyAccess::Result Result = PropertyHandle->SetValue(SelectedIp);
						if (Result != FPropertyAccess::Success)
						{
							UE_LOGF(LogTemp, Warning, "Failed to set InterfaceAddress via IPropertyHandle (Result=%d)", (int32)Result);
						}
				})
			];
	}

	void AddInterfaceAddressRow
		( IDetailChildrenBuilder& ChildBuilder
		, TSharedRef<IPropertyHandle> InterfaceHandle
		, const FString& InitialValue
		, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		IDetailPropertyRow& Row = ChildBuilder.AddProperty(InterfaceHandle);

		Row.CustomWidget()
			.NameContent()
			[
				InterfaceHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(260.f)
			[
				SNew(SRivermaxInterfaceComboBox)
					.InitialValue(InitialValue)
					.OnIPAddressSelected_Lambda([InterfaceHandle](const FString& SelectedIp)
					{
						FScopedTransaction Tx(NSLOCTEXT("Rivermax", "ChangeInterfaceAddress", "Change Interface Address"));

						const FPropertyAccess::Result Result = InterfaceHandle->SetValue(SelectedIp);
						if (Result != FPropertyAccess::Success)
						{
							UE_LOGF(LogTemp, Warning, "Failed to set InterfaceAddress via IPropertyHandle (Result=%d)", (int32)Result);
						}
					})
			];
	}
}
