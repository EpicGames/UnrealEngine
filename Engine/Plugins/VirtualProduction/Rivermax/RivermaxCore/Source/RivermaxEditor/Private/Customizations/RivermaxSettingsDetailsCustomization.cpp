// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxSettingsDetailsCustomization.h"

#include "Customizations/RivermaxDeviceSelectionCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "RivermaxSettings.h"

TSharedRef<IDetailCustomization> FRivermaxSettingsDetailsCustomization::MakeInstance()
{
	return MakeShared<FRivermaxSettingsDetailsCustomization>();
}

void FRivermaxSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	for (int32 ObjectIndex = 0; ObjectIndex < CustomizedObjects.Num(); ++ObjectIndex)
	{
		// For now, take care of both Rivermax source and output to customize their interface field
		const TWeakObjectPtr<UObject> Obj = CustomizedObjects[ObjectIndex];
		if (URivermaxSettings* RivermaxSettings = Cast<URivermaxSettings>(Obj.Get()))
		{
			UE::RivermaxCore::Utils::SetupDeviceSelectionCustomization(ObjectIndex, RivermaxSettings->PTPInterfaceAddress, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(URivermaxSettings, PTPInterfaceAddress)), DetailBuilder);
		}
	}
}
