// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"


class IPropertyHandle;

class FRivermaxSettingsDetailsCustomization : public IDetailCustomization
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface
};
