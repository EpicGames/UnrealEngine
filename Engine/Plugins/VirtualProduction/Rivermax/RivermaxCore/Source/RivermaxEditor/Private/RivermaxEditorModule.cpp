// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxEditorModule.h"

#include "Customizations/RivermaxSettingsDetailsCustomization.h"
#include "PropertyEditorModule.h"
#include "RivermaxSettings.h"


void FRivermaxEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const auto AddCustomization = [&PropertyModule, this](const FName ClassName, FOnGetDetailCustomizationInstance Customizor)
	{
		PropertyModule.RegisterCustomClassLayout(ClassName, Customizor);
		CustomizedClasses.Add(ClassName);
	};

	AddCustomization(URivermaxSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FRivermaxSettingsDetailsCustomization::MakeInstance));
}

void FRivermaxEditorModule::ShutdownModule()
{
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule)
	{
		for (FName ClassName : CustomizedClasses)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassName);
		}
	}
}

IMPLEMENT_MODULE(FRivermaxEditorModule, RivermaxEditor);
