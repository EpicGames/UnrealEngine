// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRivermaxInterfaceComboBox.h"

#include "EditorStyleSet.h"
#include "IRivermaxCoreModule.h"
#include "IRivermaxManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "RivermaxComboBox"


namespace UE::RivermaxEditor::Private
{
	static TArray<TSharedPtr<FString>> GetRivermaxNetworkInterfaceCardIPs()
	{
		TArray<TSharedPtr<FString>> RivermaxDeviceIPs;

		IRivermaxCoreModule& RivermaxModule = FModuleManager::LoadModuleChecked<IRivermaxCoreModule>(TEXT("RivermaxCore"));
		for(const UE::RivermaxCore::FRivermaxDeviceInfo& Device : RivermaxModule.GetRivermaxManager()->GetDevices())
		{
			RivermaxDeviceIPs.Add(MakeShared<FString>(Device.InterfaceAddress));
		}
		
		return RivermaxDeviceIPs;
	}
}

void SRivermaxInterfaceComboBox::Construct(const FArguments& InArgs)
{
	OnIPAddressSelectedDelegate = InArgs._OnIPAddressSelected;
	RivermaxAdapterAddressSource = UE::RivermaxEditor::Private::GetRivermaxNetworkInterfaceCardIPs();
	ChildSlot
	[
		SAssignNew(HorizontalBoxContainer, SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(RivermaxAdapterAddressComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&RivermaxAdapterAddressSource)
			.OnGenerateWidget(this, &SRivermaxInterfaceComboBox::GenerateRivermaxAdaptersComboBoxEntry)
			.OnSelectionChanged(this, &SRivermaxInterfaceComboBox::OnIpAddressSelected)
			.Content()
			[
				SAssignNew(IPAddressEditableTextBlock, SEditableText)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromString(InArgs._InitialValue))
				.OnTextCommitted(this, &SRivermaxInterfaceComboBox::OnIPAddressTextCommmited)
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(IPAddressWarningImage, SImage)
		]
	];
	
	UpdateIPWarning();
}

void SRivermaxInterfaceComboBox::OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType)
{
	TSharedPtr<FString> SelectedIPAddress = RivermaxAdapterAddressComboBox->GetSelectedItem();
	if (SelectedIPAddress.IsValid())
	{
		IPAddressEditableTextBlock->SetText(FText::FromString(*SelectedIPAddress));
		OnIPAddressSelectedDelegate.ExecuteIfBound(*InAddress.Get());
		UpdateIPWarning();
	}
}

void SRivermaxInterfaceComboBox::OnIPAddressTextCommmited(const FText& Text, ETextCommit::Type)
{
	OnIPAddressSelectedDelegate.ExecuteIfBound(Text.ToString());
	UpdateIPWarning();
}

TSharedRef<SWidget> SRivermaxInterfaceComboBox::GenerateRivermaxAdaptersComboBoxEntry(TSharedPtr<FString> InAddress)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InAddress))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SRivermaxInterfaceComboBox::UpdateIPWarning()
{
	const FString CurrentIP = IPAddressEditableTextBlock->GetText().ToString();
	IRivermaxCoreModule& Module = FModuleManager::GetModuleChecked<IRivermaxCoreModule>("RivermaxCore");
	if (Module.GetRivermaxManager()->IsValidIP(CurrentIP))
	{
		FString MatchingDevice;
		if (Module.GetRivermaxManager()->GetMatchingDevice(CurrentIP, MatchingDevice))
		{
			HorizontalBoxContainer->SetToolTipText(FText::Format(LOCTEXT("MatchingDeviceFound", "Provided IP resolved to Rivermax device {0}"), FText::FromString(MatchingDevice)));
			IPAddressWarningImage->SetVisibility(EVisibility::Collapsed);

		}
		else
		{
			IPAddressWarningImage->SetImage(FAppStyle::Get().GetBrush("Icons.WarningWithColor"));
			HorizontalBoxContainer->SetToolTipText(LOCTEXT("MatchingDeviceNotFound", "Could not find Rivermax device matching provided IP"));
			IPAddressWarningImage->SetVisibility(EVisibility::Visible);
		}
	}
	else
	{
		IPAddressWarningImage->SetImage(FAppStyle::Get().GetBrush("Icons.ErrorWithColor"));
		HorizontalBoxContainer->SetToolTipText(LOCTEXT("InvalidIPProvided", "Invalid IP format provided. Must be of the form x.x.x.x with * accepted as wildcard."));
		IPAddressWarningImage->SetVisibility(EVisibility::Visible);
	}
}

#undef LOCTEXT_NAMESPACE

