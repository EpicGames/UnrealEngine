// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/Input/SComboBox.h"

class SEditableText;
class SEditableTextBox;
class SImage;
class SHorizontalBox;

/** 
 * Widget used to pick IP address of Rivermax / Mellanox device
 */
class RIVERMAXEDITOR_API SRivermaxInterfaceComboBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnChooseIP, FString)
	
	SLATE_BEGIN_ARGS(SRivermaxInterfaceComboBox)
	{}
	SLATE_ARGUMENT(FString, InitialValue)
		SLATE_EVENT(FOnChooseIP, OnIPAddressSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	/** Handles changes in the local adapter address combo box */
	void OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType);

	/** Called when the IP Address was commited in the editable text block */
	void OnIPAddressTextCommmited(const FText&, ETextCommit::Type);

	/** Generates an entry in the local adapter address combo box */
	TSharedRef<SWidget> GenerateRivermaxAdaptersComboBoxEntry(TSharedPtr<FString> InAddress);

	/** Updates image showing warning if IP is invalid */
	void UpdateIPWarning();

private:

	TArray<TSharedPtr<FString>> RivermaxAdapterAddressSource;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> RivermaxAdapterAddressComboBox;

	/** Text box shown on top of the local adapter address address combo box */
	TSharedPtr<SEditableText> IPAddressEditableTextBlock;

	/** Warning images displayed when IP is not valid or no device matches resolved IP */
	TSharedPtr<SImage> IPAddressWarningImage;
	
	/** Horizontal box containing combo button, text and image */
	TSharedPtr<SHorizontalBox> HorizontalBoxContainer;

	/** Delegate executed when a local IP address was selected */
	FOnChooseIP OnIPAddressSelectedDelegate;
};

