// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

namespace UE::RivermaxCore::Utils
{
	RIVERMAXEDITOR_API void SetupDeviceSelectionCustomization(int32 InObjectIndex, const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder);
	RIVERMAXEDITOR_API void AddInterfaceAddressRow(IDetailChildrenBuilder& ChildBuilder, TSharedRef<IPropertyHandle> InterfaceHandle, const FString& InitialValue, IPropertyTypeCustomizationUtils& CustomizationUtils);
}
