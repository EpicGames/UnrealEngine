// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RivermaxEditor : ModuleRules
{
	public RivermaxEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"PropertyEditor",
				"RivermaxCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"EditorStyle",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				}
			);
		}
	}
}
