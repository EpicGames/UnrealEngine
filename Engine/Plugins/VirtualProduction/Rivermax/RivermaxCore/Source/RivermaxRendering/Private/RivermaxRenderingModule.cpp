// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxRenderingModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"


void FRivermaxRenderingModule::StartupModule()
{
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("RivermaxCore"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/RivermaxCore"), PluginShaderDir);

}
IMPLEMENT_MODULE(FRivermaxRenderingModule, RivermaxRendering);
