// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


/**
 * Module used to define Rivermax shaders used for pixel format conversion
 */
class FRivermaxRenderingModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	//~ End IModuleInterface
};


