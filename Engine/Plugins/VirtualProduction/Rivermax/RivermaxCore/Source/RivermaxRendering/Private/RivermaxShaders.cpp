// Copyright Epic Games, Inc. All Rights Reserved.

#include "RivermaxShaders.h"

#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"


namespace UE::RivermaxShaders
{
	/* FYUV8Bit422ToRGBACS shader
	*****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FYUV8Bit422ToRGBACS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "YUV8Bit422ToRGBACS", SF_Compute);

	FYUV8Bit422ToRGBACS::FParameters* FYUV8Bit422ToRGBACS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef YUVBuffer, FRDGTextureRef OutputTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, int32 TotalElementCount, int32 BufferLineCount)
	{
		FYUV8Bit422ToRGBACS::FParameters* Parameters = GraphBuilder.AllocParameters<FYUV8Bit422ToRGBACS::FParameters>();

		Parameters->InputYUV4228bitBuffer= GraphBuilder.CreateSRV(YUVBuffer);
		Parameters->ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}

	/* FRGBToYUV8Bit422CS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToYUV8Bit422CS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToYUV8Bit422", SF_Compute);

	FRGBToYUV8Bit422CS::FParameters* FRGBToYUV8Bit422CS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGBufferRef OutputBuffer)
	{
		FRGBToYUV8Bit422CS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToYUV8Bit422CS::FParameters>();

		Parameters->RGBToYUVConversion.InputTexture = SourceTexture;
		Parameters->RGBToYUVConversion.InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->RGBToYUVConversion.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		Parameters->RGBToYUVConversion.DoLinearToSrgb = bDoLinearToSrgb;
		Parameters->RGBToYUVConversion.OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;

		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		// Output size will be based on RGB to YUV422 encoding ( divide by 2 ) 
		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutYUV4228bitBuffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FYUV10Bit422ToRGBACS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FYUV10Bit422ToRGBACS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "YUV10Bit422ToRGBACS", SF_Compute);

	FYUV10Bit422ToRGBACS::FParameters* FYUV10Bit422ToRGBACS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef YUVBuffer, FRDGTextureRef OutputTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, int32 TotalElementCount, int32 BufferLineCount)
	{
		FYUV10Bit422ToRGBACS::FParameters* Parameters = GraphBuilder.AllocParameters<FYUV10Bit422ToRGBACS::FParameters>();

		Parameters->InputYCbCrBuffer = GraphBuilder.CreateSRV(YUVBuffer);
		Parameters->ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}

	/* FRGBToYUV10Bit422LittleEndianCS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToYUV10Bit422LittleEndianCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToYUV10Bit422", SF_Compute);

	FRGBToYUV10Bit422LittleEndianCS::FParameters* FRGBToYUV10Bit422LittleEndianCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGBufferRef OutputBuffer)
	{
		FRGBToYUV10Bit422LittleEndianCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToYUV10Bit422LittleEndianCS::FParameters>();

		Parameters->RGBToYUVConversion.InputTexture = SourceTexture;
		Parameters->RGBToYUVConversion.InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->RGBToYUVConversion.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		Parameters->RGBToYUVConversion.DoLinearToSrgb = bDoLinearToSrgb;
		Parameters->RGBToYUVConversion.OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;
		
		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		// Output size will be based on RGB to YUV encoding ( divide by 2 ) and pixels encoded per element ( 4 pixels )
		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutYCbCrBuffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FRGB8BitToRGBA8CS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGB8BitToRGBA8CS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGB8BitToRGBA8CS", SF_Compute);

	FRGB8BitToRGBA8CS::FParameters* FRGB8BitToRGBA8CS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount)
	{
		FRGB8BitToRGBA8CS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB8BitToRGBA8CS::FParameters>();

		Parameters->InputRGB8Buffer = GraphBuilder.CreateSRV(RGBBuffer);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}

	/* FRGBToRGB8BitCS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToRGB8BitCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToRGB8BitCS", SF_Compute);

	FRGBToRGB8BitCS::FParameters* FRGBToRGB8BitCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer)
	{
		FRGBToRGB8BitCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToRGB8BitCS::FParameters>();

		Parameters->InputTexture = SourceTexture;
		Parameters->InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;

		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutRGB8Buffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FRGB10BitToRGBA10CS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGB10BitToRGBA10CS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGB10BitToRGBACS", SF_Compute);


	FRGB10BitToRGBA10CS::FParameters* FRGB10BitToRGBA10CS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount)
	{
		FRGB10BitToRGBA10CS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB10BitToRGBA10CS::FParameters>();

		Parameters->InputBuffer = GraphBuilder.CreateSRV(RGBBuffer);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}

	/* FRGBToRGB10BitCS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToRGB10BitCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToRGB10BitCS", SF_Compute);


	FRGBToRGB10BitCS::FParameters* FRGBToRGB10BitCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer)
	{
		FRGBToRGB10BitCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToRGB10BitCS::FParameters>();

		Parameters->InputTexture = SourceTexture;

		Parameters->InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;

		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutRGB10Buffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FRGB12BitToRGBA12CS shader
	*****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGB12BitToRGBA12CS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGB12BitToRGBACS", SF_Compute);


	FRGB12BitToRGBA12CS::FParameters* FRGB12BitToRGBA12CS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount)
	{
		FRGB12BitToRGBA12CS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB12BitToRGBA12CS::FParameters>();

		Parameters->InputRGB12Buffer = GraphBuilder.CreateSRV(RGBBuffer);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}

	/* FRGBToRGB12BitCS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToRGB12BitCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToRGB12BitCS", SF_Compute);


	FRGBToRGB12BitCS::FParameters* FRGBToRGB12BitCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer)
	{
		FRGBToRGB12BitCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToRGB12BitCS::FParameters>();

		Parameters->InputTexture = SourceTexture;
		Parameters->InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;

		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutRGB12Buffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FRGBToRGB16fCS shader
	 *****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGBToRGB16fCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGBToRGB16fBitCS", SF_Compute);


	FRGBToRGB16fCS::FParameters* FRGBToRGB16fCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer)
	{
		FRGBToRGB16fCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGBToRGB16fCS::FParameters>();

		Parameters->InputTexture = SourceTexture;
		Parameters->InputSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		Parameters->OnePixelDeltaX = 1.0f / (float)SourceTexture->Desc.Extent.X;
		Parameters->OnePixelDeltaY = 1.0f / (float)SourceTexture->Desc.Extent.Y;

		// Used to offset indices to sample input texture based on ThreadId of CS
		Parameters->CapturedSizeX = CaptureSize.X;
		Parameters->InputPixelOffsetX = SourceViewRect.Min.X;
		Parameters->InputPixelOffsetY = SourceViewRect.Min.Y;

		Parameters->TotalElementCount = OutputSize.X;

		Parameters->OutRGB16fBuffer = GraphBuilder.CreateUAV(OutputBuffer);

		return Parameters;
	}

	/* FRGB16fBitToRGBA16fCS shader
	*****************************************************************************/

	IMPLEMENT_GLOBAL_SHADER(FRGB16fBitToRGBA16fCS, "/Plugin/RivermaxCore/Private/RivermaxShaders.usf", "RGB16fBitToRGBACS", SF_Compute);


	FRGB16fBitToRGBA16fCS::FParameters* FRGB16fBitToRGBA16fCS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount)
	{
		FRGB16fBitToRGBA16fCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB16fBitToRGBA16fCS::FParameters>();

		Parameters->InputRGB16fBuffer = GraphBuilder.CreateSRV(RGBBuffer);
		Parameters->TotalElementCount = TotalElementCount;
		Parameters->OutputTextureSizeX = OutputTexture->Desc.Extent.X;
		Parameters->OutTexture = GraphBuilder.CreateUAV(OutputTexture, ERDGUnorderedAccessViewFlags::None);

		return Parameters;
	}
}

