// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"

#include "MediaShaders.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Shader.h"
#include "ShaderParameterStruct.h"


namespace UE::RivermaxShaders
{
	/**
	 * Compute shader to convert 2110 YUV8 to RGBA
	 */
	class FYUV8Bit422ToRGBACS : public FGlobalShader
	{
	public:

		//Structure definition to match StructuredBuffer in RivermaxShaders.usf
		struct FYUV8Bit422Buffer
		{
			uint32 DWord0;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FYUV8Bit422ToRGBACS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FYUV8Bit422ToRGBACS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FYUV8Bit422Buffer>, InputYUV4228bitBuffer)
			SHADER_PARAMETER(FMatrix44f, ColorTransform)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FYUV8Bit422ToRGBACS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef YUVBuffer, FRDGTextureRef OutputTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, int32 TotalElementCount, int32 BufferLineCount);
	};

	/**
	 * Compute shader to convert RGB to YUV422 8 bits
	 */
	class FRGBToYUV8Bit422CS : public FGlobalShader
	{
	public:
		//Structure definition to match StructuredBuffer in RivermaxShaders.usf
		struct FYUV8Bit422Buffer
		{
			uint32 DWord0;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToYUV8Bit422CS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToYUV8Bit422CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FRGBToYUVConversion, RGBToYUVConversion)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FYUV8Bit422Buffer>, OutYUV4228bitBuffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToYUV8Bit422CS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert packed 8 bits RGB to RGBA 8bits
	 */
	class FYUV10Bit422ToRGBACS : public FGlobalShader
	{
	public:

		//Structure definition to match StructuredBuffer in RivermaxShaders.usf
		struct FYUV10Bit422LEBuffer
		{
			uint32 DWord0;
			uint32 DWord1;
			uint32 DWord2;
			uint32 DWord3;
			uint32 DWord4;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FYUV10Bit422ToRGBACS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FYUV10Bit422ToRGBACS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FYUV10Bit422LEBuffer>, InputYCbCrBuffer)
			SHADER_PARAMETER(FMatrix44f, ColorTransform)	
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FYUV10Bit422ToRGBACS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef YUVBuffer, FRDGTextureRef OutputTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, int32 TotalElementCount, int32 BufferLineCount);
	};

	/**
	 * Compute shader to convert RGB to packed YUV422 10 bits little endian
	 *
	 * This shader expects a single texture in PF_A2B10G10R10 format.
	 */
	class FRGBToYUV10Bit422LittleEndianCS : public FGlobalShader
	{
	public:
		//Structure definition to match StructuredBuffer in RivermaxShaders.usf
		struct FYUV10Bit422LEBuffer
		{
			uint32 DWord0;
			uint32 DWord1;
			uint32 DWord2;
			uint32 DWord3;
			uint32 DWord4;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToYUV10Bit422LittleEndianCS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToYUV10Bit422LittleEndianCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_INCLUDE(FRGBToYUVConversion, RGBToYUVConversion)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FYUV10Bit422LEBuffer>, OutYCbCrBuffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToYUV10Bit422LittleEndianCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert packed 8 bits RGB to RGBA 8bits
	 */
	class FRGB8BitToRGBA8CS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// For 8bit output, it's 3 bytes per pixels. To align with 4 bytes (32bit) we use 12 bytes (4 pixels) per output
		struct FRGB8BitBuffer
		{
			uint32 DWord0;
			uint32 DWord1;
			uint32 DWord2;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGB8BitToRGBA8CS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FRGB8BitToRGBA8CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRGB8BitBuffer>, InputRGB8Buffer)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGB8BitToRGBA8CS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount);
	};

	/**
	 * Compute shader to convert RGBA to packed 8 bits RGB
	 */
	class FRGBToRGB8BitCS : public FGlobalShader
	{
	public:
		
		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// For 8bit output, it's 3 bytes per pixels. To align with 4 bytes (32bit) we use 12 bytes (4 pixels) per output
		struct FRGB8BitBuffer
		{
			uint32 DWord0;
			uint32 DWord1;
			uint32 DWord2;
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToRGB8BitCS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToRGB8BitCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(float, OnePixelDeltaX)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRGB8BitBuffer>, OutRGB8Buffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToRGB8BitCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert RGBA to packed 10 bits RGB
	 */
	class FRGBToRGB10BitCS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// 10 bits per channel, 30 bits per pixel. Aligns on 15 32bits -> 480bits -> 16RGB10 pixels
		// 2110-20 says it's 4 pixels per pgroup. 
		struct FRGB10BitBuffer
		{
			uint32 DWords[15];
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToRGB10BitCS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToRGB10BitCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(float, OnePixelDeltaX)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRGB10BitBuffer>, OutRGB10Buffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToRGB10BitCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert packed 10 bits RGB to RGBA
	 */
	class FRGB10BitToRGBA10CS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// 10 bits per channel, 30 bits per pixel. Aligns on 15 32bits -> 480bits -> 16RGB10 pixels
		// 2110-20 says it's 4 pixels per pgroup. 
		struct FRGB10BitBuffer
		{
			uint32 DWords[15];
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGB10BitToRGBA10CS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FRGB10BitToRGBA10CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRGB10BitBuffer>, InputBuffer)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGB10BitToRGBA10CS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount);
	};

	/**
	 * Compute shader to convert RGBA to packed 12 bits RGB
	 */
	class FRGBToRGB12BitCS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// 12 bits per channel, 36 bits per pixel. Aligns on 9 32bits -> 288bits -> 8x RGB12 pixels
		// 2110-20 says it's 2 pixels per pgroup. 
		struct FRGB12BitBuffer
		{
			uint32 DWords[9];
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToRGB12BitCS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToRGB12BitCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(float, OnePixelDeltaX)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRGB12BitBuffer>, OutRGB12Buffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToRGB12BitCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert packed 12 bits RGB to RGBA
	 */
	class FRGB12BitToRGBA12CS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// 12 bits per channel, 36 bits per pixel. Aligns on 9 32bits -> 288bits -> 8x RGB12 pixels
		// 2110-20 says it's 2 pixels per pgroup. 
		struct FRGB12BitBuffer
		{
			uint32 DWords[9];
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGB12BitToRGBA12CS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FRGB12BitToRGBA12CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRGB12BitBuffer>, InputRGB12Buffer)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGB12BitToRGBA12CS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount);
	};

	/**
	 * Compute shader to convert RGBA to Float16 RGB
	 */
	class FRGBToRGB16fCS : public FGlobalShader
	{
	public:

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGBToRGB16fCS, RIVERMAXRENDERING_API);

		SHADER_USE_PARAMETER_STRUCT(FRGBToRGB16fCS, FGlobalShader);

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// To align on 32bits, use 2 pixels (16 bit * 3 * 2 = 3x32bits)
		struct FRGB16fBuffer
		{
			uint32 DWords[3];
		};

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			SHADER_PARAMETER(uint32, CapturedSizeX)
			SHADER_PARAMETER(float, OnePixelDeltaX)
			SHADER_PARAMETER(float, OnePixelDeltaY)
			SHADER_PARAMETER(float, InputPixelOffsetX)
			SHADER_PARAMETER(float, InputPixelOffsetY)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRGB16fBuffer>, OutRGB16fBuffer)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGBToRGB16fCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, const FIntPoint& CaptureSize, const FIntRect& SourceViewRect, const FIntPoint& OutputSize, FRDGBufferRef OutputBuffer);
	};

	/**
	 * Compute shader to convert 16 bits float RGB to RGBA
	 */
	class FRGB16fBitToRGBA16fCS : public FGlobalShader
	{
	public:

		// Structure definition to match StructuredBuffer in RivermaxShaders.usf
		// To align on 32bits, use 2 pixels (16 bit * 3 * 2 = 3x32bits)
		struct FRGB16fBuffer
		{
			uint32 DWords[3];
		};

		DECLARE_EXPORTED_GLOBAL_SHADER(FRGB16fBitToRGBA16fCS, RIVERMAXRENDERING_API);
		SHADER_USE_PARAMETER_STRUCT(FRGB16fBitToRGBA16fCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRGB16fBuffer>, InputRGB16fBuffer)
			SHADER_PARAMETER(uint32, TotalElementCount)
			SHADER_PARAMETER(uint32, OutputTextureSizeX)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

	public:

		// Called by the engine to determine which permutations to compile for this shader
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		RIVERMAXRENDERING_API FRGB16fBitToRGBA16fCS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGBufferRef RGBBuffer, FRDGTextureRef OutputTexture, int32 TotalElementCount, int32 BufferLineCount);
	};
}

