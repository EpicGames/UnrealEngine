// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RivermaxRendering : ModuleRules
{
	public RivermaxRendering(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Projects",
				"RenderCore",
				"RHI"
			}
		);
	}
}
